{
	"FileVersion": 3,
	"Version": 1.0,
	"VersionName": "1.0",
	"FriendlyName": "MQTT",
	"Description": "MQTT broker and client",
	"Category": "IOT",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Modules": [
		{
			"Name": "MQTTCore",
			"Type": "Runtime"
		},
		{
			"Name": "MQTTCoreEditor",
			"Type": "Editor"
		}
	],
	"Plugins": [
		{
			"Name": "JsonBlueprintUtilities",
			"Enabled": true
		}
	]
}
