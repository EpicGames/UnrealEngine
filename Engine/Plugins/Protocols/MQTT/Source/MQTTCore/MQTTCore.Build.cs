// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MQTTCore : ModuleRules
{
	public MQTTCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new[]
			{
				"Core",
				"CoreUObject",
				"JsonBlueprintUtilities",
				"JsonUtilities",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new[]
			{
				"Engine",
				"Json",
				"JsonBlueprintUtilities",
				"Networking",
				"Projects",
				"Sockets"
			}
		);
		
		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new[] {
					"FunctionalTesting",
					"UnrealEd"
				});
		}
	}
}
