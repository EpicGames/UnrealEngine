// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTClient.h"

#include "MQTTClientMessage.h"
#include "MQTTConnection.h"
#include "MQTTCoreLog.h"
#include "MQTTOperations.h"
#include "SocketSubsystem.h"
#include "Algo/ForEach.h"
#include "Async/Async.h"

FMQTTClient::FMQTTClient(const FMQTTURL& InURL)
	: ClientId(InURL.ToGuid())
	, URL(InURL)
{
	FText URLMessage;
	if(!InURL.IsValid(URLMessage))
	{
		UE_LOGF(LogMQTTCore, Error, "URL is not valid: %ls", *URLMessage.ToString());
		return;
	}

	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	TSharedPtr<FInternetAddr> InternetAddr;
	if(!URL.GetAddress(SocketSubsystem, InternetAddr))
	{
		UE_LOGF(LogMQTTCore, Error, "Cannot create MQTTConnection: Invalid IP address: %ls", *InURL.Host);
		URL.SetInvalid();
	}
	else
	{
		Connection = FMQTTConnection::TryCreate(InternetAddr);
		Connection->OnMessage().BindRaw(this, &FMQTTClient::OnMessagePacket);
	} 
}

FMQTTClient::~FMQTTClient()
{
	if(Connection.IsValid())
	{
		Connection->OnMessage().Unbind();
		Connection.Reset();
	}
}

TFuture<EMQTTConnectReturnCode> FMQTTClient::Connect(const bool bCleanSession)
{
	TFuture<EMQTTConnectReturnCode> Result = MakeFulfilledPromise<EMQTTConnectReturnCode>(URL.IsValid() ? EMQTTConnectReturnCode::AlreadyConnected : EMQTTConnectReturnCode::InvalidURL).GetFuture();;
	if(Connection.IsValid())
	{
		const auto ConnectionState = Connection->GetState();
		if(Connection->IsConnected())
		{
			Result = MakeFulfilledPromise<EMQTTConnectReturnCode>(EMQTTConnectReturnCode::AlreadyConnected).GetFuture();
		}
		else if(ConnectionState == EMQTTState::None)
		{
			Result = MakeFulfilledPromise<EMQTTConnectReturnCode>(EMQTTConnectReturnCode::SocketError).GetFuture();
		}
		// Not currently connecting
		else if(ConnectionState != EMQTTState::Connecting)
		{
			// Set immediately to prevent multiple requests
			Connection->SetState(EMQTTState::Connecting);
			UE_LOGF(LogTemp, VeryVerbose, "QUEUED CONNECT");
			Result = Connection->QueueOperation<FMQTTConnectOperation>(
					FMQTTConnectPacket(
						URL.ToGuid().ToString(),
						60,
						URL.Username.IsEmpty() ? "" : URL.Username,
						URL.Password.IsEmpty() ? "" : URL.Password,
						bCleanSession))
				.Next([Self = TWeakPtr<FMQTTClient, ESPMode::ThreadSafe>(AsShared())](const FMQTTConnectAckPacket InResponse)
				{
					UE_LOGF(LogTemp, VeryVerbose, "GOT CONNACK");
					if(!Self.IsValid())
					{
						return EMQTTConnectReturnCode::SocketError;
					}
					
					if (InResponse.ReturnCode > EMQTTConnectReturnCode::Accepted && InResponse.ReturnCode != EMQTTConnectReturnCode::AlreadyConnected)
					{
						UE_LOGF(LogMQTTCore, Warning, "Couldn't connect to MQTT server: %ls", MQTT::GetMQTTConnectReturnCodeDescription(InResponse.ReturnCode));
					}

					Self.Pin()->Connection->SetState(EMQTTState::Connected);
					return InResponse.ReturnCode;
				});
		}
	}

	return Result.Next([Self = TSharedPtr<FMQTTClient, ESPMode::ThreadSafe>(AsShared())](EMQTTConnectReturnCode InCode)
	{
		if(!Self.IsValid())
		{
			return EMQTTConnectReturnCode::SocketError;
		}
		
		Self->OnConnect().Broadcast(InCode);
		return InCode;
	});
}

TFuture<void> FMQTTClient::Disconnect()
{
	if(Connection.IsValid())
	{
		// Check if connecting/connecting
		if(Connection->GetState() > EMQTTState::Connecting)
		{
			return MakeFulfilledPromise<void>().GetFuture();
		}
	
		// Set immediately to prevent multiple requests
		Connection->SetState(EMQTTState::Disconnecting);
		return Connection->QueueOperation<FMQTTDisconnectOperation>(FMQTTDisconnectPacket())
			.Next([Self = TWeakPtr<FMQTTClient, ESPMode::ThreadSafe>(AsShared())]()
			{
				if (!Self.IsValid())
				{
					return;
				}

				if(const auto SharedSelf = Self.Pin())
				{
					SharedSelf->Connection->SetState(EMQTTState::Disconnected);
					SharedSelf->OnDisconnect().Broadcast();
				}
			});
	}

	return MakeFulfilledPromise<void>().GetFuture();
}

TFuture<bool> FMQTTClient::Publish(const FString& InTopic, const FString& InPayload, const EMQTTQualityOfService InQoS, const bool bInRetain)
{
	return Publish(InTopic, TArray<uint8>((uint8*)TCHAR_TO_UTF8(*InPayload), InPayload.Len()), InQoS, bInRetain);
}

TFuture<bool> FMQTTClient::Publish(const FString& InTopic, const TArray<uint8>& InPayload, const EMQTTQualityOfService InQoS, const bool bInRetain)
{
	if(Connection.IsValid())
	{
		FMQTTPublishPacket PublishPacket(
			Connection->PopId(),
			InTopic,
			InPayload,
			InQoS,
			bInRetain);

		if(PublishPacket.Topic.IsEmpty())
		{
			UE_LOGF(LogMQTTCore, Warning, "Attempted to publish message without topic");
			return MakeFulfilledPromise<bool>(false).GetFuture();			
		}

		if(!FMQTTTopic::IsValid(InTopic))
		{
			UE_LOGF(LogMQTTCore, Warning, "Attempted to publish with an invalid or malformed topic: \'%ls\'", *InTopic);
			return MakeFulfilledPromise<bool>(false).GetFuture();			
		}

		TFuture<FMQTTPublishAckPacket> PublishResponse;
		switch(InQoS)
		{
		case EMQTTQualityOfService::Once:
			return Connection->QueueOperation<FMQTTPublishOperationQoS0>(MoveTemp(PublishPacket))
			.Next([Self = TWeakPtr<FMQTTClient, ESPMode::ThreadSafe>(AsShared())]()
			{
				if(!Self.IsValid())
				{
					return false;
				} 
				
				Self.Pin()->OnPublish().Broadcast();
				return true;
			});
			
		case EMQTTQualityOfService::AtLeastOnce:
			return Connection->QueueOperation<FMQTTPublishOperationQoS1>(MoveTemp(PublishPacket))
			.Next([Self = TWeakPtr<FMQTTClient, ESPMode::ThreadSafe>(AsShared())](FMQTTPublishAckPacket /*InResponse*/)
			{
				if(!Self.IsValid())
				{
					return false;
				} 
				
				Self.Pin()->OnPublish().Broadcast();
				return true;
			});
			
		case EMQTTQualityOfService::ExactlyOnce:
			return Connection->QueueOperation<FMQTTPublishOperationQoS2>(MoveTemp(PublishPacket))
			.Next([Self = TWeakPtr<FMQTTClient, ESPMode::ThreadSafe>(AsShared())](FMQTTPublishCompletePacket /*InResponse*/)
			{
				if(!Self.IsValid())
				{
					return false;
				} 
				
				Self.Pin()->OnPublish().Broadcast();
				return true;
			});
			
		default: ;
		}
	}

	return MakeFulfilledPromise<bool>(false).GetFuture();
}

TFuture<TArray<FMQTTSubscribeResult>> FMQTTClient::Subscribe(const TArray<TPair<FString, EMQTTQualityOfService>>& InTopicFilterQoSPairs)
{
	static TArray<FMQTTSubscribeResult> FailureResponse = { FMQTTSubscribeResult(EMQTTSubscribeReturnCode::Failure, nullptr) };
	
	if(InTopicFilterQoSPairs.IsEmpty())
	{
		UE_LOGF(LogMQTTCore, Warning, "Attempted to subscribe without specifying a topic");		
		return MakeFulfilledPromise<TArray<FMQTTSubscribeResult>>(FailureResponse).GetFuture();
	}
	
	if(Connection.IsValid())
	{
		TArray<FString> TopicFilters;
		TopicFilters.Reserve(InTopicFilterQoSPairs.Num());

		TArray<EMQTTQualityOfService> TopicQoS;
		TopicQoS.Reserve(InTopicFilterQoSPairs.Num());

		for(const TPair<FString, EMQTTQualityOfService>& TopicFilterQoSPair : InTopicFilterQoSPairs)
		{
			if(TopicFilterQoSPair.Key.IsEmpty())
			{
				UE_LOGF(LogMQTTCore, Warning, "Attempted to subscribe with an empty topic");
				return MakeFulfilledPromise<TArray<FMQTTSubscribeResult>>(FailureResponse).GetFuture();	
			}

			if(!FMQTTTopic::IsValid(TopicFilterQoSPair.Key))
			{
				UE_LOGF(LogMQTTCore, Warning, "Attempted to subscribe with an invalid or malformed topic: \'%ls\'", *TopicFilterQoSPair.Key);
				return MakeFulfilledPromise<TArray<FMQTTSubscribeResult>>(FailureResponse).GetFuture();
			}
			
			TopicFilters.Add(TopicFilterQoSPair.Key);
			TopicQoS.Add(TopicFilterQoSPair.Value);		
		}

		const uint16 PacketId = Connection->PopId();

		UE_LOGF(LogMQTTCore, Verbose, "Queued Subscribe message with PacketId %d., and Topic Filter: \'%ls\'", PacketId, *TopicFilters[0]);
		
		return Connection->QueueOperation<FMQTTSubscribeOperation>(
			FMQTTSubscribePacket(
				PacketId,
				TopicFilters,
				TopicQoS))
		.Next([Self = TWeakPtr<FMQTTClient, ESPMode::ThreadSafe>(AsShared()), InTopicFilterQoSPairs](FMQTTSubscribeAckPacket InResponse)
		{
			TArray<FMQTTSubscribeResult> Result;
			if(!Self.IsValid())
			{
				return Result;
			}

			if(auto SharedSelf = Self.Pin())
			{
				Result.Reserve(InResponse.ReturnCodes.Num());

				// Was abandoned
				if(InResponse.ReturnCodes.IsEmpty())
				{
					return Result;				
				}
			
				ensure(InResponse.ReturnCodes.Num() == InTopicFilterQoSPairs.Num());
				for(int32 Idx = 0; Idx < FMath::Min(InResponse.ReturnCodes.Num(), InTopicFilterQoSPairs.Num()); ++Idx)
				{
					const TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe> Subscription = SharedSelf->MakeSubscription(InTopicFilterQoSPairs[Idx].Key, static_cast<EMQTTQualityOfService>(InResponse.ReturnCodes[Idx]));
					Result.Add({static_cast<EMQTTSubscribeReturnCode>(InResponse.ReturnCodes[Idx]), Subscription});
				}

				SharedSelf->OnSubscribe().Broadcast(Result);
			}			

			return Result;
		});
	}

	return MakeFulfilledPromise<TArray<FMQTTSubscribeResult>>(FailureResponse).GetFuture();
}

TFuture<bool> FMQTTClient::Unsubscribe(const TSet<FString>& InTopicFilters)
{
	if(Connection.IsValid())
	{
		return Connection->QueueOperation<FMQTTUnsubscribeOperation>(
			FMQTTUnsubscribePacket(
				Connection->PopId(),
				InTopicFilters.Array()))
		.Next([Self = TWeakPtr<FMQTTClient, ESPMode::ThreadSafe>(AsShared())](FMQTTUnsubscribeAckPacket InResponse)
		{
			if(!Self.IsValid())
			{
				return false;
			} 
			
			Self.Pin()->OnUnsubscribe().Broadcast();
			return true;
		});
	}

	return MakeFulfilledPromise<bool>(false).GetFuture();
}

TFuture<bool> FMQTTClient::Ping(const float& InTimeout)
{
	if(Connection.IsValid())
	{
		return Connection->QueueOperation<FMQTTPingOperation>({})
		.Next([](FMQTTPingResponsePacket InResponse)
		{
			return true;
		});
	}

	return MakeFulfilledPromise<bool>(false).GetFuture();
}

bool FMQTTClient::IsConnected() const
{
	return Connection.IsValid() && Connection->GetState() == EMQTTState::Connected;
}

bool FMQTTClient::IsValid() const
{
	return ClientId.IsValid() && URL.IsValid();
}

TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe>& FMQTTClient::MakeSubscription(const FString& InTopicFilter, EMQTTQualityOfService InGrantedQoS)
{
	FScopeLock Lock(&SubscriptionsLock);
	return Subscriptions.Add(
		FName(*InTopicFilter),
		MakeShared<FMQTTSubscription, ESPMode::ThreadSafe>(InTopicFilter, InGrantedQoS));
}

// Handle raw incoming publish packet
void FMQTTClient::OnMessagePacket(const FMQTTPublishPacket& InPacket)
{
	FMQTTClientMessage Message;
	Message.TimeStamp = FDateTime::UtcNow();
	Message.Topic = InPacket.Topic;
	Message.Payload = InPacket.Payload;
	Message.QoS = InPacket.QoS;
	Message.bRetain = InPacket.bRetain;
	Message.PayloadString = FString::ConstructFromPtrSize((const char*)Message.Payload.GetData(), Message.Payload.Num());

	// No subscriptions, can happen if that overload isn't used
	if(Subscriptions.Num() == 0)
	{
		OnMessage().Broadcast(Message);
	}
	else
	{
		// Route message
		Async(EAsyncExecution::TaskGraph, [&, Message]()
		{		
			TArray<TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe>> SubscriptionsCopy;
			{
				FScopeLock Lock(&SubscriptionsLock);
				Subscriptions.GenerateValueArray(SubscriptionsCopy);
			}

			OnMessage().Broadcast(Message);

			FString Topic = Message.Topic;
			Algo::ForEachIf(
				SubscriptionsCopy,
				[&Topic](const auto& InSubscription) { return InSubscription->Matches(Topic); },
				[&Message](const auto& InSubscription)
				{
					InSubscription->OnSubscriptionMessage().Broadcast(Message);
				});
		});
	}
}
