// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMQTTClient.h"

class FMQTTConnection;
enum class EMQTTConnectReturnCode : uint8;
struct FMQTTPublishPacket;

class FMQTTClient final
	: public IMQTTClient
	, public TSharedFromThis<FMQTTClient, ESPMode::ThreadSafe>
{
public:
	explicit FMQTTClient(const FMQTTURL& InURL);
	virtual ~FMQTTClient() override;

	//~ Begin IMQTTClient Interface
	virtual FOnConnect& OnConnect() override { return OnConnectDelegate; }
	virtual FOnDisconnect& OnDisconnect() override { return OnDisconnectDelegate; }
	virtual FOnPublish& OnPublish() override { return OnPublishDelegate; }
	virtual FOnMessage& OnMessage() override { return OnMessageDelegate; }
	virtual FOnSubscribe& OnSubscribe() override { return OnSubscribeDelegate; }
	virtual FOnUnsubscribe& OnUnsubscribe() override { return OnUnsubscribeDelegate; }

	virtual FGuid GetClientId() const override { return ClientId; }

	virtual const FMQTTURL& GetURL() const override	{ return URL; }	

	virtual TFuture<EMQTTConnectReturnCode> Connect(bool bCleanSession = true) override;
	virtual TFuture<void> Disconnect() override;

	virtual TFuture<bool> Publish(const FString& InTopic, const TArray<uint8>& InPayload, EMQTTQualityOfService InQoS = EMQTTQualityOfService::Once, const bool bInRetain = false) override;
	virtual TFuture<bool> Publish(const FString& InTopic, const FString& InPayload, EMQTTQualityOfService InQoS = EMQTTQualityOfService::Once, const bool bInRetain = false) override;
	virtual TFuture<TArray<FMQTTSubscribeResult>> Subscribe(const TArray<TPair<FString, EMQTTQualityOfService>>& InTopicFilterQoSPairs) override;
	virtual TFuture<bool> Unsubscribe(const TSet<FString>& InTopicFilters) override;
	virtual TFuture<bool> Ping(const float& InTimeout = 2.0f) override;

	virtual bool IsConnected() const override;

	virtual bool IsValid() const override;
	//~ End IMQTTClient Interface

protected:
	TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe>& MakeSubscription(const FString& InTopicFilter, EMQTTQualityOfService InGrantedQoS);
	void OnMessagePacket(const FMQTTPublishPacket& InPacket);

private:
	TSharedPtr<FMQTTConnection, ESPMode::ThreadSafe> Connection;
	
	FOnConnect OnConnectDelegate;
	FOnDisconnect OnDisconnectDelegate;	
	FOnPublish OnPublishDelegate;	
	FOnSubscribe OnSubscribeDelegate;
	FOnUnsubscribe OnUnsubscribeDelegate;
	FOnMessage OnMessageDelegate;

	FGuid ClientId;
	FMQTTURL URL;

	mutable FCriticalSection SubscriptionsLock;
	TMap<FName, TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe>> Subscriptions;
};
