// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTClientMessage.h"

#include "Serialization/JsonSerializer.h"

bool FMQTTClientMessage::GetPayloadAsJson(TSharedPtr<FJsonObject>& OutJson) const
{
	return FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(PayloadString), OutJson);
}

void FMQTTClientMessage::SetPayloadFromString(const FString& InPayloadString)
{
	PayloadString = InPayloadString;
	Payload = TArray<uint8>((uint8*)TCHAR_TO_UTF8(*InPayloadString), InPayloadString.Len());
}
