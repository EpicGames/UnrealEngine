// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTClientObject.h"

#include "IMQTTClient.h"
#include "IMQTTCoreModule.h"
#include "MQTTClientMessage.h"
#include "MQTTCoreLog.h"

//~ Begin UMQTTSubscriptionObject
UMQTTSubscriptionObject::~UMQTTSubscriptionObject()
{
	if (const TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe> MQTTSubscription = MqttSubscriptionPtr)
	{
		MQTTSubscription->OnSubscriptionMessage().RemoveAll(this);
	}
}

void UMQTTSubscriptionObject::Initialize(const TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe>& InSubscription)
{
	check(InSubscription.IsValid());

	MqttSubscriptionPtr = InSubscription;
	MqttSubscriptionPtr->OnSubscriptionMessage().AddUObject(this, &UMQTTSubscriptionObject::OnMessage);
}

void UMQTTSubscriptionObject::SetOnMessageHandler(const FOnMessageDelegate& OnMessageCallback)
{
	OnMessageDelegate = OnMessageCallback;
}

void UMQTTSubscriptionObject::OnMessage(const FMQTTClientMessage& InMessage) const
{
	OnMessageDelegate.ExecuteIfBound(InMessage);
}
//~ End UMQTTSubscriptionObject

UMQTTClientObject::~UMQTTClientObject()
{
	RemoveDelegates();
	if(MqttClientPtr.IsValid())
	{
		MqttClientPtr.Reset();
	}
}

void UMQTTClientObject::Initialize(const FMQTTURL& InURL)
{
	const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = IMQTTCoreModule::Get().GetOrCreateClient(InURL);
	if (MQTTClient)
	{
		MqttClientPtr = MQTTClient;
		InitDelegates();
	}
}

void UMQTTClientObject::InitDelegates()
{
	if (const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = MqttClientPtr)
	{
		// MQTTClient->OnConnect().AddUObject(this, &UMQTTClientObject::OnConnect);
		// MQTTClient->OnDisconnect().AddUObject(this, &UMQTTClientObject::OnDisconnect);
		// MQTTClient->OnPublish().AddUObject(this, &UMQTTClientObject::OnPublish);
		// MQTTClient->OnMessage().AddUObject(this, &UMQTTClientObject::OnMessage);
		// MQTTClient->OnSubscribe().AddLambda([&](TArray<FMQTTSubscribeResult> InSubscriptions)
		// {
		// 	// Get list of granted quality of service levels, one per topic subscribed to (usually only one)
		// 	TArray<EMQTTQualityOfService> QoS;
		// 	Algo::Transform(
		// 		InSubscriptions,
		// 		QoS,
		// 		[](const FMQTTSubscribeResult& InItem)
		// 		{
		// 			return InItem.Subscription->GetGrantedQoS();
		// 		});
		// 	
		// 	OnSubscribe(QoS);
		// });
		// MQTTClient->OnUnsubscribe().AddUObject(this, &UMQTTClientObject::OnUnsubscribe);
	}
}

void UMQTTClientObject::RemoveDelegates() const
{
	if (const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = MqttClientPtr)
	{
		MQTTClient->OnConnect().RemoveAll(this);
		MQTTClient->OnDisconnect().RemoveAll(this);
		MQTTClient->OnPublish().RemoveAll(this);
		MQTTClient->OnMessage().RemoveAll(this);
		MQTTClient->OnSubscribe().RemoveAll(this);
		MQTTClient->OnUnsubscribe().RemoveAll(this);
	}
}

void UMQTTClientObject::BeginDestroy()
{
	if (const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = MqttClientPtr)
	{
		//IMQTTCoreModule::Get().DestroyClient(MQTTClient);
	}
	
	Super::BeginDestroy();
}

TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> UMQTTClientObject::GetClientImpl() const
{
	return MqttClientPtr;
}

void UMQTTClientObject::Connect(const FOnConnectDelegate& InOnConnect)
{
	if (const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = MqttClientPtr)
	{
		MQTTClient->Connect()
		.Next([&](EMQTTConnectReturnCode InReturnCode)
		{
			if(IsValidChecked(this))
			{
				InOnConnect.ExecuteIfBound(InReturnCode);
			}
		});
	}
}

void UMQTTClientObject::Disconnect(const FOnDisconnectDelegate& InOnDisconnect)
{
	OnDisconnectDelegate = InOnDisconnect;
	if (const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = MqttClientPtr)
	{
		const FGuid ClientId = MQTTClient->GetClientId();
		MQTTClient->Disconnect()
		.Next([&]()
		{
			if(IsValidChecked(this))
			{
				InOnDisconnect.ExecuteIfBound();
			}
		});
	}
}

void UMQTTClientObject::Publish(
	const FString& InTopic,
	const TArray<uint8>& InPayload,
	EMQTTQualityOfService InQoS,
	const bool bInRetain)
{
	if (const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = MqttClientPtr)
	{
		MQTTClient->Publish(InTopic, InPayload, InQoS, bInRetain);
	}
}

UMQTTSubscriptionObject* UMQTTClientObject::Subscribe(const FString& InTopic, EMQTTQualityOfService InQoS)
{
	if (const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = MqttClientPtr)
	{
		UMQTTSubscriptionObject* SubscriptionObject = NewObject<UMQTTSubscriptionObject>(this);
		TObjectPtr<UMQTTSubscriptionObject>& SubscriptionObjectRef = this->Subscriptions.Add_GetRef(SubscriptionObject);

		MQTTClient->Subscribe({MakeTuple(InTopic, InQoS)})
		.Next([&](TArray<FMQTTSubscribeResult> InSubscribeResults)
		{
			if(IsValidChecked(this))
			{
				if(InSubscribeResults.IsEmpty() || InSubscribeResults[0].ReturnCode == EMQTTSubscribeReturnCode::Failure)
				{
					return;
				}

				TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe> Subscription = InSubscribeResults[0].Subscription;
				SubscriptionObjectRef->Initialize(Subscription);
				// Subscription->OnSubscriptionMessage() // @todo: get from user callbacks?
			}
		});

		return SubscriptionObject;
	}

	return nullptr;
}

TArray<UMQTTSubscriptionObject*> UMQTTClientObject::SubscribeMany(
	const TArray<FString>& InTopics,
	const TArray<EMQTTQualityOfService> InQoS)
{
	if (const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = MqttClientPtr)
	{
		ensure(InTopics.Num() == InQoS.Num());
		
		// @todo: how to prevent same topic returning duplicate subscription
		TArray<UMQTTSubscriptionObject*> Result;
		Result.Reserve(InTopics.Num());

		TArray<TPair<FString, EMQTTQualityOfService>> TopicQoSPairs;
		TopicQoSPairs.Reserve(InTopics.Num());
		
		for(auto Idx = 0; Idx < InTopics.Num(); ++Idx)
		{
			UMQTTSubscriptionObject* SubscriptionObject = NewObject<UMQTTSubscriptionObject>(this);
			Subscriptions.Add(SubscriptionObject);
			Result[Idx] = SubscriptionObject;

			TopicQoSPairs[Idx] = MakeTuple(InTopics[Idx], InQoS[Idx]);
		}

		MQTTClient->Subscribe(TopicQoSPairs)
		.Next([&](TArray<FMQTTSubscribeResult> InSubscribeResults)
		{
			if(IsValidChecked(this))
			{
				if(InSubscribeResults.IsEmpty() || InSubscribeResults[0].ReturnCode == EMQTTSubscribeReturnCode::Failure)
				{
					return;
				}

				for(auto Idx = 0; Idx < InSubscribeResults.Num(); ++Idx)
				{
					TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe> Subscription = InSubscribeResults[Idx].Subscription;
					auto* SubscriptionObject = Subscriptions.FindByPredicate([&](const UMQTTSubscriptionObject* InSubscription)
					{
						return IsValid(InSubscription) && GetTypeHash(InSubscription) == GetTypeHash(Subscription.Get());
					});
					
					if(!SubscriptionObject)
					{
						UE_LOGF(LogMQTTCore, Error, "Failed to find SubscriptionObject for topic: '%ls'", *Subscription->GetTopic().ToString());
						continue;
					}
					
					(*SubscriptionObject)->Initialize(Subscription);
				}
			}
		});
	}

	return {};
}

void UMQTTClientObject::Unsubscribe(const FString& InTopic)
{
	if (const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = MqttClientPtr)
	{
		MQTTClient->Unsubscribe({InTopic});
	}
}

FString UMQTTClientObject::GetClientId() const
{
	if (const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = MqttClientPtr)
	{
		return MQTTClient->GetClientId().ToString();
	}

	return FString();
}

FMQTTURL UMQTTClientObject::GetURL() const
{
	if (const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = MqttClientPtr)
	{
		return MQTTClient->GetURL();
	}

	return {};
}
