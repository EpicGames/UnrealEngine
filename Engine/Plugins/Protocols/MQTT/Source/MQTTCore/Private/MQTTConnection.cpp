// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTConnection.h"

#include "MQTTClientSettings.h"
#include "MQTTCoreLog.h"
#include "MQTTCoreStats.h"
#include "MQTTOperations.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "HAL/RunnableThread.h"
#include "Serialization/ArrayReader.h"
#include "Stats/StatsTrace.h"

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("MQTT Packets Sent Total"), STAT_MQTTPackagesSent, STATGROUP_MQTT);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("MQTT Packets Received Total"), STAT_MQTTPackagesReceived, STATGROUP_MQTT);
DECLARE_FLOAT_COUNTER_STAT(TEXT("MQTT Execution Delay Average"), STAT_MQTTExecutionTime, STATGROUP_MQTT);

#define LOCTEXT_NAMESPACE "MQTTConnection"

FMQTTPacketIdGenerator::FMQTTPacketIdGenerator()
	: Min(1)
	, Position(1)
{
	for(auto Idx = 0; Idx < ValueStore.Num(); ++Idx)
	{
		ValueStore[Idx] = Idx;
	}	
}

uint16 FMQTTPacketIdGenerator::Pop()
{
	// Shift so it's in single range (shifted pos -> max)
	const uint16 ShiftedPosition = (Position++) - Min;
	// Then shift back to origin, and wrap
	return ValueStore[(ShiftedPosition + Min) % Max];		
}

void FMQTTPacketIdGenerator::Push(const uint16 InValue)
{
	ValueStore[Min++] = InValue;
}

FMQTTConnection::FMQTTConnection(
	FSocket& InSocket,
	const TSharedRef<FInternetAddr> InAddr)
	: Socket(&InSocket)
	, EndpointInternetAddr(InAddr)
	, Thread(nullptr)
{
	check(Socket->GetSocketType() == SOCKTYPE_Streaming);

	ThreadName = FString(TEXT("MQTTConnection_")) + InAddr->ToString(true);
	Thread = FRunnableThread::Create(this, *ThreadName, 0, TPri_TimeCritical, FPlatformAffinity::GetPoolThreadMask());

	UE_LOGF(LogMQTTCore, VeryVerbose, "Created MQTTConnection for %ls", *EndpointInternetAddr->ToString(false));
}

FMQTTConnection::~FMQTTConnection()
{
	SetState(EMQTTState::Stopping); 

	AbandonOperations();

	if (Socket)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		SocketSubsystem->DestroySocket(Socket);
	}
	
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
		Thread = nullptr;
	}

	if (Socket)
	{
		Socket = nullptr;
	}

	UE_LOGF(LogMQTTCore, VeryVerbose, "Destroyed MQTTConnection at %ls", *EndpointInternetAddr->ToString(false));
}

TSharedPtr<FMQTTConnection, ESPMode::ThreadSafe> FMQTTConnection::TryCreate(const TSharedPtr<FInternetAddr>& InAddr)
{
	if(!InAddr.IsValid() || !InAddr->IsValid())
	{
		UE_LOGF(LogMQTTCore, Error, "Cannot create MQTTConnection, the IP Address was invalid.");
		return nullptr;		
	}
	
	ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
	FSocket* Socket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("MQTTConnection"), FNetworkProtocolTypes::IPv4);

	if (!Socket)
	{
		UE_LOGF(LogMQTTCore, Error, "Cannot create MQTTConnection: Error creating Socket for: %ls: %ls", *InAddr->ToString(true), SocketSubsystem->GetSocketError(SocketSubsystem->GetLastErrorCode()));
		return nullptr;
	}

	Socket->SetReuseAddr(true);

	TSharedPtr<FMQTTConnection, ESPMode::ThreadSafe> Runnable = MakeShared<FMQTTConnection, ESPMode::ThreadSafe>(*Socket, InAddr.ToSharedRef());
	return Runnable;
}

bool FMQTTConnection::Init()
{
	return FRunnable::Init();
}

uint32 FMQTTConnection::Run()
{
	const UMQTTClientSettings* MQTTSettings = GetDefault<UMQTTClientSettings>();
	check(MQTTSettings);

	// Fixed rate delta time
	const double SendDeltaTime = 1.0f / MQTTSettings->PublishRate;

	while (GetState() != EMQTTState::Stopping)
	{
		const double StartTime = FPlatformTime::Seconds();

		if (UpdateConnection())
		{
			ProcessMessages();
		}

		const double EndTime = FPlatformTime::Seconds();
		const double WaitTime = SendDeltaTime - (EndTime - StartTime);

		if (WaitTime > 0.f)
		{
			// Sleep by the amount which is set in refresh rate
			FPlatformProcess::SleepNoStats(WaitTime);
		}
	}

	return 0;
}

void FMQTTConnection::Stop()
{
	AbandonOperations();
	SetState(EMQTTState::Stopping);
}

void FMQTTConnection::Exit()
{
	FRunnable::Exit();	
}

void FMQTTConnection::Tick()
{
	ProcessMessages();
}

FSingleThreadRunnable* FMQTTConnection::GetSingleThreadInterface()
{
	return this;
}

bool FMQTTConnection::UpdateConnection()
{
	EMQTTSocketState CurrentSocketState = SocketState;
	if(CurrentSocketState == EMQTTSocketState::Connected)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		const ESocketErrors LastErrorCode = SocketSubsystem->GetLastErrorCode();
		if(LastErrorCode > ESocketErrors::SE_EINPROGRESS)
		{
			CurrentSocketState = EMQTTSocketState::Error;	
		}
	}

	switch(CurrentSocketState)
	{
	// Already connected, skip
	case EMQTTSocketState::Connected:
		break;

	case EMQTTSocketState::ConnectionRejected:
	case EMQTTSocketState::Stopping:
		{
			// Unless connecting, abandon queued operations
			if(CurrentSocketState != EMQTTSocketState::Connecting)
			{
				if(PendingConnectOperation.IsValid())
				{
					PendingConnectOperation->Complete(FMQTTConnectAckPacket(EMQTTConnectReturnCode::SocketError));
				}
			
				AbandonOperations();
			}
		
			return false;
		}

	case EMQTTSocketState::Connecting:
	case EMQTTSocketState::Error:
	default:
		{
			const ESocketConnectionState ConnectionState = Socket->GetConnectionState();
			if(ConnectionState == ESocketConnectionState::SCS_Connected)
			{
				SocketState = EMQTTSocketState::Connected;
			}
			// Socket isn't connected
			else
			{
				Socket->SetNonBlocking(true);
				OpenSocket();
				Socket->SetNonBlocking(false);

				// If there's an error, abandon all operations until next explicit connect call
				if(SocketState == EMQTTSocketState::Error)
				{
					if(PendingConnectOperation.IsValid())
					{
						PendingConnectOperation->Complete(FMQTTConnectAckPacket(EMQTTConnectReturnCode::SocketError));
					}
			
					AbandonOperations();
				}

				return false;
			}
		}
	}

	// Check MQTT Connect timeout
	if(GetState() == EMQTTState::Connecting && PendingConnectOperation.IsValid())
	{
		if(FPlatformTime::Seconds() > LastConnectRequestTime + MQTTConnectionTimeout)
		{
			PendingConnectOperation->Complete(FMQTTConnectAckPacket(EMQTTConnectReturnCode::SocketError));
			PendingConnectOperation.Reset();
			SetState(EMQTTState::Disconnected);
			return false;
		}
	}

	return true;
}

void FMQTTConnection::ProcessMessages()
{
	if(GetState() == EMQTTState::Stopping)
	{
		return;
	}
	
	// KeepAlive ping
	if(FPlatformTime::Seconds() >= LastKeepAliveTime)
	{
		QueueOperation<FMQTTPingOperation>({})
		.Next([](FMQTTPingResponsePacket InResponse)
		{
			// @todo: something if no response
			return true;
		});
		
		LastKeepAliveTime = FPlatformTime::Seconds() + KeepAliveInterval;
	}

	TQueue<TUniquePtr<IMQTTOperation>, EQueueMode::Spsc> PendingOutgoingOperations;
	{
		UE_LOGF(LogMQTTCore, Verbose, "Copy outgoing operations to buffer");
		
		uint32 DeferredOperationCount = 0;
		
		// Copy to temporary buffer
		FScopeLock Lock(&OutgoingQueueLock);
		TUniquePtr<IMQTTOperation> Operation;
		TQueue<TUniquePtr<IMQTTOperation>> DeferredOperations;
		while(OutgoingOperations.Dequeue(Operation))
		{
			// Was set in previous tick, skip
			if(!Operation.IsValid() || !Operation)
			{
				continue;
			}
			
			if(Operation->StateDependency == EMQTTState::None || Operation->StateDependency == GetState())
			{
				PendingOutgoingOperations.Enqueue(MoveTemp(Operation));
			}
			else
			{
				DeferredOperationCount++;
				DeferredOperations.Enqueue(MoveTemp(Operation));
			}
		}

		UE_LOGF(LogMQTTCore, Verbose, "Operations deferred: %d", DeferredOperationCount);

		// Move deferred back into queue for future processing
		while(DeferredOperations.Dequeue(Operation))
		{
			OutgoingOperations.Enqueue(MoveTemp(Operation));
		}
	}

	TQueue<TUniquePtr<IMQTTOperation>, EQueueMode::Spsc> PendingIncoming;

	// Send outgoing packets
	TUniquePtr<IMQTTOperation> Operation;
	while(PendingOutgoingOperations.Dequeue(Operation))
	{
#if WITH_EDITORONLY_DATA
		Operation->MarkExecuted();
		SET_FLOAT_STAT(STAT_MQTTExecutionTime, Operation->GetTimeToExecution());
#endif
		
		FBufferArchive BufferArchive;
		Operation->Pack(BufferArchive);
		
		int32 ExpectedBytesSent = BufferArchive.Num();
		int32 ActualBytesSent = -1;

 		if(Socket->SendTo(BufferArchive.GetData(), BufferArchive.Num(), ActualBytesSent, *EndpointInternetAddr))
		{
			INC_DWORD_STAT(STAT_MQTTPackagesSent);			
		}
		else
		{
			ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
			UE_LOGF(LogMQTTCore, Error, "Error sending MQTT packet: %ls: %ls", *EndpointInternetAddr->ToString(true), SocketSubsystem->GetSocketError(SocketSubsystem->GetLastErrorCode()));
		}

		if(ActualBytesSent != ExpectedBytesSent)
		{
			UE_LOGF(LogMQTTCore, Error, "Packet size mismatch: expected %d, was %d", ExpectedBytesSent, ActualBytesSent);
		}

		if(Operation->GetPendingPacketType() != EMQTTPacketType::None)
		{
			PendingIncoming.Enqueue(MoveTemp(Operation));
		}
		// Packet doesn't have a response, so set promise to indicate it's been processed 
		else
		{
			uint16 ReturnedMessageId = Operation->Complete();
			PushId(MoveTemp(ReturnedMessageId));
		}
	}

	// Set outgoing operations with expected requests to PendingIncoming
	if(!PendingIncoming.IsEmpty())
	{
		FScopeLock IncomingLock(&IncomingQueueLock);
		while(PendingIncoming.Dequeue(Operation))
		{
#if UE_BUILD_DEBUG
			UE_LOGF(LogMQTTCore, Verbose, "Re-queue pending incoming operation: %ls", *Operation->GetTypeName());
#endif
			const EMQTTPacketType MessageType = Operation->GetPacketType();
			if(MessageType == EMQTTPacketType::Connect)
			{
				// There should only be one Connect request at a time!
				if(PendingConnectOperation.IsValid())
				{
					checkNoEntry();
				}
				
				// @note: connection request made, so record timestamp
				LastConnectRequestTime = FPlatformTime::Seconds();				
				
				PendingConnectOperation = TUniquePtr<FMQTTConnectOperation>(StaticCast<FMQTTConnectOperation*>(Operation.Release()));
			}
			else if(MessageType == EMQTTPacketType::PingRequest) 
			{
				// If already waiting, cancel
				if(PendingPingOperation.IsValid())
				{
					PendingPingOperation->Abandon();
					PendingPingOperation.Reset();
				}
				PendingPingOperation = MoveTemp(Operation);
			}
			else
			{
				uint16 PacketId = Operation->GetPacketId();
				PendingIncomingOperations.Emplace(PacketId, MoveTemp(Operation));
			}
		}
	}

	// Wait until something to do
	if (Socket && !Socket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromSeconds(0.1f)))
	{
		return;
	}

	// Process any incoming packets
	uint32 Size = 0;
	while (GetState() != EMQTTState::Stopping && Socket && Socket->HasPendingData(Size))
	{
		UE_LOGF(LogMQTTCore, Verbose, "Processing incoming packets of size: %d", Size);
		
		TSharedRef<FArrayReader> Reader = MakeShared<FArrayReader>(true);
		Reader->SetNumUninitialized(Size);

		int32 Read = 0;
		if (Socket->RecvFrom(Reader->GetData(), Reader->Num(), Read, *EndpointInternetAddr))
		{
			// ensure((uint32)Read <= Size);
			// Reader->RemoveAt(Read, Reader->Num() - Read, false);

			int64 ReaderPos = Reader->Tell();
			while(ReaderPos < Size)
			{
				FMQTTFixedHeader Header;
				*Reader << Header;
				Reader->Seek(ReaderPos); // go back to start of packet
			
				HandleMessage(Header.GetPacketType(), Reader);

				ReaderPos = Reader->Tell();
				
				INC_DWORD_STAT(STAT_MQTTPackagesReceived);
			}
		}
	}
}

void FMQTTConnection::OpenSocket()
{
	if(Socket)
	{
		SocketState = EMQTTSocketState::Connecting;
		
		const double TimeoutTime = FPlatformTime::Seconds() + SocketConnectionTimeout;
		do
		{
			if(Socket->Connect(*EndpointInternetAddr))
			{
				const ESocketConnectionState SocketConnectionState = Socket->GetConnectionState();
				SocketState = SocketConnectionState == SCS_NotConnected
								? EMQTTSocketState::Connecting
								: static_cast<EMQTTSocketState>(SocketConnectionState); // ESocketConnectionState maps to EMQTTSocketState
				return;
			}
			FPlatformProcess::SleepNoStats(0.5f);
 		}
		while(FPlatformTime::Seconds() < TimeoutTime && GetState() != EMQTTState::Stopping);

		if(SocketState != EMQTTSocketState::Stopping)
		{
			SocketState = EMQTTSocketState::Error;
			ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
			const ESocketErrors LastErrorCode = SocketSubsystem->GetLastErrorCode();
			UE_LOGF(LogMQTTCore, Error, "Error connecting socket to: %ls: %ls, Connection state: %ls",
				*EndpointInternetAddr->ToString(true),
				SocketSubsystem->GetSocketError(LastErrorCode),
				MQTT::GetSocketConnectionStateName(Socket->GetConnectionState()));

			if(LastErrorCode == SE_ECONNREFUSED
				|| LastErrorCode == SE_EINVAL
				|| LastErrorCode == SE_ETIMEDOUT)
			{
				// Don't attempt again unless explicit Connect call
				SocketState = EMQTTSocketState::ConnectionRejected;

				// If connection pending, cancel
				if(PendingConnectOperation.IsValid())
				{
					PendingConnectOperation->Complete(FMQTTConnectAckPacket(EMQTTConnectReturnCode::SocketError));
				}

				AbandonOperations();
			}
		}
	}
}

void FMQTTConnection::AbandonOperations()
{
	UE_LOGF(LogMQTTCore, Verbose, "Abandoning Operations");
	
	{
		FScopeLock Lock(&OutgoingQueueLock);
		TUniquePtr<IMQTTOperation> Operation;
		while(OutgoingOperations.Dequeue(Operation))
		{
			Operation->Abandon();
		}
		OutgoingOperations.Empty();
	}

	{
		FScopeLock Lock(&IncomingQueueLock);
		//TArray<TUniquePtr<IMQTTOperation>> Messages;
		//PendingIncomingOperations.GenerateValueArray(Messages);
		//for(const TUniquePtr<IMQTTOperation>& Message : Messages)
		//{
		//	Message->Abandon();
		//}
		PendingIncomingOperations.Empty();
	}

	if(PendingConnectOperation.IsValid())
	{
		PendingConnectOperation->Abandon();
		PendingConnectOperation.Reset();
	}

	if(PendingPingOperation.IsValid())
	{
		PendingPingOperation->Abandon();
		PendingPingOperation.Reset();
	}
}

void FMQTTConnection::HandleMessage(const EMQTTPacketType InMessageType, const TSharedRef<FArrayReader>& InPacketReader)
{
	switch(InMessageType)
	{
	case EMQTTPacketType::ConnectAcknowledge:
		HandleConnectAcknowledge(InPacketReader);
		break;

	case EMQTTPacketType::SubscribeAcknowledge: 
		HandleSubscribeAcknowledge(InPacketReader);
		break;
	
	case EMQTTPacketType::UnsubscribeAcknowledge:
		HandleUnsubscribeAcknowledge(InPacketReader);
		break;

	// Incoming published message
	case EMQTTPacketType::Publish:
		HandlePublish(InPacketReader);
		break;

	// Publish QoS 1
	case EMQTTPacketType::PublishAcknowledge:
		HandlePublishAcknowledge(InPacketReader);
		break;
		
	// Publish QoS 2: Step 2
	case EMQTTPacketType::PublishReceived:
		HandlePublishReceived(InPacketReader);
		break;

	// Publish QoS 2: Step 3
	case EMQTTPacketType::PublishRelease:
		HandlePublishRelease(InPacketReader);
		break;

	// Publish QoS 2: Step 4
	case EMQTTPacketType::PublishComplete:
		HandlePublishComplete(InPacketReader);
		break;

	case EMQTTPacketType::PingResponse:
		HandlePingResponse(InPacketReader);
		break;
		
	case EMQTTPacketType::None:
		{
			UE_LOGF(LogMQTTCore, Error, "PacketType was None");
		}
		break;
		
	default:
		{
			UE_LOGF(LogMQTTCore, Error, "Unhandled, unknown message type: %d (%ls)", int(InMessageType), MQTT::GetMQTTPacketTypeName(InMessageType));
		}
	}
}

void FMQTTConnection::HandleConnectAcknowledge(const TSharedRef<FArrayReader>& InPacketReader)
{
	FMQTTConnectAckPacket Message;
	*InPacketReader << Message;
	{
		FScopeLock IncomingLock(&IncomingQueueLock);
		TUniquePtr<IMQTTOperation> Operation = MoveTemp(PendingConnectOperation);
		if(const auto CastMessageTask = StaticCast<TMQTTOperation<FMQTTConnectPacket, FMQTTConnectAckPacket>*>(Operation.Release()))
		{
			CastMessageTask->Complete(MoveTemp(Message));
			PendingConnectOperation.Reset();
		}
	}
	UE_LOGF(LogMQTTCore, VeryVerbose, "Handled ConnectAck message.");
}

void FMQTTConnection::HandlePublishAcknowledge(const TSharedRef<FArrayReader>& InPacketReader)
{
	FMQTTPublishAckPacket Message;
	*InPacketReader << Message;
	{
		FScopeLock IncomingLock(&IncomingQueueLock);
		if(const TUniquePtr<IMQTTOperation>* Operation = PendingIncomingOperations.Find(Message.PacketId))
		{
			const uint16 PacketId = Message.PacketId;
			if(const auto CastMessageTask = StaticCast<TMQTTOperation<FMQTTPublishPacket, FMQTTPublishAckPacket>*>(Operation->Get()))
			{
				CastMessageTask->Complete(MoveTemp(Message));
			}
			PendingIncomingOperations.Remove(PacketId);
			PushId(PacketId);
		}
	}
	UE_LOGF(LogMQTTCore, VeryVerbose, "Handled PublishAck message.");
}

void FMQTTConnection::HandlePublishReceived(const TSharedRef<FArrayReader>& InPacketReader)
{
	FMQTTPublishReceivedPacket Message;
	*InPacketReader << Message;
	{
		FScopeLock IncomingLock(&IncomingQueueLock);
		const uint16 PacketId = Message.PacketId;
		if(PendingIncomingOperations.Contains(PacketId))
		{
			TUniquePtr<IMQTTOperation> Operation = PendingIncomingOperations.FindAndRemoveChecked(PacketId);
			if(const auto& CastMessageTask = StaticCast<FMQTTPublishOperationQoS2*>(Operation.Get()))
			{
				CastMessageTask->CompleteStep(MoveTemp(Message));
				// Move from incoming to outgoing operations (PUBREL)
				{
					FScopeLock Lock(&OutgoingQueueLock);
					OutgoingOperations.Enqueue(MoveTemp(Operation));
				}
				PendingIncomingOperations.Remove(PacketId);
			}
		}
	}
	UE_LOGF(LogMQTTCore, VeryVerbose, "Handled PublishReceive message.");
}

void FMQTTConnection::HandlePublishRelease(const TSharedRef<FArrayReader>& InPacketReader)
{
	FMQTTPublishReleasePacket Message;
	*InPacketReader << Message;
	{
		FScopeLock IncomingLock(&IncomingQueueLock);
		const uint16 PacketId = Message.PacketId;
		if(PendingIncomingOperations.Contains(PacketId))
		{
			TUniquePtr<IMQTTOperation> Operation = PendingIncomingOperations.FindAndRemoveChecked(PacketId);			
			if(const auto& CastMessageTask = StaticCast<FMQTTPublishOperationQoS2*>(Operation.Get()))
			{
				CastMessageTask->CompleteStep(MoveTemp(Message));
				// Move from incoming to outgoing operations (PUBCOMP)
				{
					FScopeLock Lock(&OutgoingQueueLock);
					OutgoingOperations.Enqueue(MoveTemp(Operation));
				}
				PendingIncomingOperations.Remove(PacketId);
			}
		}
	}
	UE_LOGF(LogMQTTCore, VeryVerbose, "Handled PublishRelease message.");
}

void FMQTTConnection::HandlePublishComplete(const TSharedRef<FArrayReader>& InPacketReader)
{
	FMQTTPublishCompletePacket Message;
	*InPacketReader << Message;
	{
		FScopeLock IncomingLock(&IncomingQueueLock);
		if(const TUniquePtr<IMQTTOperation>* Operation = PendingIncomingOperations.Find(Message.PacketId))
		{
			const uint16 PacketId = Message.PacketId;
			if(const auto CastMessageTask = StaticCast<TMQTTOperation<FMQTTPublishReleasePacket, FMQTTPublishCompletePacket>*>(Operation->Get()))
			{
				CastMessageTask->Complete(MoveTemp(Message));
			}
			PendingIncomingOperations.Remove(PacketId);
			PushId(PacketId);
		}
	}
	UE_LOGF(LogMQTTCore, VeryVerbose, "Handled PublishComplete message.");
}

void FMQTTConnection::HandleSubscribeAcknowledge(const TSharedRef<FArrayReader>& InPacketReader)
{
	FMQTTSubscribeAckPacket Message;
	*InPacketReader << Message;
	{
		FScopeLock IncomingLock(&IncomingQueueLock);		
		if(const TUniquePtr<IMQTTOperation>* Operation = PendingIncomingOperations.Find(Message.PacketId))
		{
			if(const auto CastMessageTask = StaticCast<TMQTTOperation<FMQTTSubscribePacket, FMQTTSubscribeAckPacket>*>(Operation->Get()))
			{
				CastMessageTask->Complete(MoveTemp(Message));
			}
			PendingIncomingOperations.Remove(Message.PacketId);
			PushId(Message.PacketId);
		}
	}
	UE_LOGF(LogMQTTCore, VeryVerbose, "Handled SubscribeAck message with PacketId %d.", Message.PacketId);
}

void FMQTTConnection::HandleUnsubscribeAcknowledge(const TSharedRef<FArrayReader>& InPacketReader)
{
	FMQTTUnsubscribeAckPacket Message;
	*InPacketReader << Message;
	{
		FScopeLock IncomingLock(&IncomingQueueLock);
		if(const TUniquePtr<IMQTTOperation>* Operation = PendingIncomingOperations.Find(Message.PacketId))
		{	
			if(const auto CastMessageTask = StaticCast<TMQTTOperation<FMQTTUnsubscribePacket, FMQTTUnsubscribeAckPacket>*>(Operation->Get()))
			{
				CastMessageTask->Complete(MoveTemp(Message));
			}
			PendingIncomingOperations.Remove(Message.PacketId);
			PushId(Message.PacketId);
		}
	}
	UE_LOGF(LogMQTTCore, VeryVerbose, "Handled UnsubscribeAck message.");
}

void FMQTTConnection::HandlePingResponse(const TSharedRef<FArrayReader>& InPacketReader)
{
	FMQTTPingResponsePacket Message;
	*InPacketReader << Message;
	{
		FScopeLock IncomingLock(&IncomingQueueLock);
		const TUniquePtr<IMQTTOperation>& Operation = PendingPingOperation;
		if(const auto CastMessageTask = StaticCast<TMQTTOperation<FMQTTPingRequestPacket, FMQTTPingResponsePacket>*>(Operation.Get()))
		{
			CastMessageTask->Complete(MoveTemp(Message));
		}
	}
	UE_LOGF(LogMQTTCore, VeryVerbose, "Handled PingResponse message.");
}

void FMQTTConnection::HandlePublish(const TSharedRef<FArrayReader>& InPacketReader)
{
	FMQTTPublishPacket Message;
	*InPacketReader << Message;
	{
		// @todo: if qos > 0, setup response chain
		OnMessage().ExecuteIfBound(Message);
		
		// FScopeLock IncomingLock(&IncomingQueueLock);
		// const TSharedPtr<IMQTTOperation, ESPMode::ThreadSafe> Operation = PendingPingOperation;
		// if(const auto CastMessageTask = StaticCastSharedPtr<TMQTTOperation<FMQTTPublishPacket, FMQTTPingResponsePacket>>(Operation))
		// {
		// 	CastMessageTask->Complete(MoveTemp(Message));
		// }
	}
	UE_LOGF(LogMQTTCore, VeryVerbose, "Handled Publish message.");
}

void FMQTTConnection::SetState(const EMQTTState InState)
{
	// Can't exit stopping state
	if(GetState() == EMQTTState::Stopping)
	{
		return;
	}
	
	UE_LOGF(LogMQTTCore, Verbose, "Set State to: %ls", MQTT::GetMQTTStateName(InState));
	MQTTState.store(InState);
}

uint16 FMQTTConnection::PopId()
{
	return MessageIdGenerator.Pop();
}

void FMQTTConnection::PushId(const uint16 InValue)
{
	MessageIdGenerator.Push(InValue);
}

bool FMQTTConnection::IsConnected() const
{
	return GetState() == EMQTTState::Connected && (Socket && Socket->GetConnectionState() == SCS_Connected);
}

#undef LOCTEXT_NAMESPACE // "MQTTConnection"
