// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/Queue.h"
#include "Containers/StaticArray.h"
#include "Delegates/Delegate.h"
#include "HAL/Runnable.h"
#include "Misc/SingleThreadRunnable.h"

#if UE_BUILD_DEBUG
#include "MQTTCoreLog.h"
#include "MQTTSharedInternal.h"
#endif

class FArrayReader;
class FInternetAddr;
class FSocket;
class IMQTTOperation;
enum class EMQTTPacketType : uint8;
enum class EMQTTSocketState : uint8;
enum class EMQTTState : uint8;
struct FMQTTConnectAckPacket;
struct FMQTTConnectPacket;
struct FMQTTPublishPacket;
template <typename RequestType, typename ResponseType> class TMQTTOperation;

/** A wrapped unique (per client) id generator. */
class FMQTTPacketIdGenerator
{
public:
	FMQTTPacketIdGenerator();
	
	uint16 Pop();

	void Push(const uint16 InValue);

protected:
	static constexpr uint16 Max = TNumericLimits<uint16>::Max();
	std::atomic<uint16> Min; // Current index of next slot to be returned
	std::atomic<uint16> Position; // Current index of slot ready to go
	TStaticArray<uint16, Max> ValueStore;
};

/** Socket connection. */
class MQTTCORE_API FMQTTConnection final
	: public FRunnable
	, public FSingleThreadRunnable
	, public TSharedFromThis<FMQTTConnection, ESPMode::ThreadSafe>
{
protected:
	// Allows MakeShared with private constructor
	friend class SharedPointerInternals::TIntrusiveReferenceController<FMQTTConnection, ESPMode::ThreadSafe>;
	
	/** Constructor. Hidden on purpose, use Create instead. */
	explicit FMQTTConnection(
		FSocket& InSocket,
		TSharedRef<FInternetAddr> InAddr);

public:
	virtual ~FMQTTConnection() override;

	static TSharedPtr<FMQTTConnection, ESPMode::ThreadSafe> TryCreate(const TSharedPtr<FInternetAddr>& InAddr);

	template <class OperationType>
	auto QueueOperation(typename OperationType::FRequestType&& InRequestPacket) -> TFuture<typename OperationType::FResponseType>;

	EMQTTState GetState() const { return MQTTState.load(); }
	void SetState(const EMQTTState InState);

	/** Get new message Id. */
	uint16 PopId();

	/** Add message Id back to pool. */
	void PushId(uint16 InValue);

	bool IsConnected() const;

	DECLARE_DELEGATE_OneParam(FOnMessage, const FMQTTPublishPacket& /* Packet */)
	FOnMessage& OnMessage() { return OnMessageDelegate; }	
	
protected:
	//~ Begin FRunnable implementation
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;
	//~ End FRunnable implementation

	//~ Begin FSingleThreadRunnable implementation
	virtual void Tick() override;
	virtual class FSingleThreadRunnable* GetSingleThreadInterface() override;
	//~ End FSingleThreadRunnable implementation

	/** Updates the socket connection and resulting state. Return true if connected. */
	bool UpdateConnection();
	void ProcessMessages();

	void OpenSocket();
	
	FCriticalSection OutgoingQueueLock;
	FCriticalSection IncomingQueueLock;
	TQueue<TUniquePtr<IMQTTOperation>, EQueueMode::Spsc> OutgoingOperations;
	TUniquePtr<TMQTTOperation<FMQTTConnectPacket, FMQTTConnectAckPacket>> PendingConnectOperation;
	TUniquePtr<IMQTTOperation> PendingPingOperation; 
	TMap<uint16, TUniquePtr<IMQTTOperation>> PendingIncomingOperations; // Waiting for ack's, etc.
	
	std::atomic<EMQTTSocketState> SocketState;
	std::atomic<EMQTTState> MQTTState;

	FCriticalSection MessageIdGeneratorLock;
	FMQTTPacketIdGenerator MessageIdGenerator;

private:
	void AbandonOperations();
	void HandleMessage(EMQTTPacketType InMessageType, const TSharedRef<FArrayReader>& InPacketReader);
	void HandleConnectAcknowledge(const TSharedRef<FArrayReader>& InPacketReader);
	void HandleSubscribeAcknowledge(const TSharedRef<FArrayReader>& InPacketReader);
	void HandleUnsubscribeAcknowledge(const TSharedRef<FArrayReader>& InPacketReader);
	void HandlePingResponse(const TSharedRef<FArrayReader>& InPacketReader);
	void HandlePublish(const TSharedRef<FArrayReader>& InPacketReader);
	void HandlePublishAcknowledge(const TSharedRef<FArrayReader>& InPacketReader);
	void HandlePublishReceived(const TSharedRef<FArrayReader>& InPacketReader);
	void HandlePublishRelease(const TSharedRef<FArrayReader>& InPacketReader);	
	void HandlePublishComplete(const TSharedRef<FArrayReader>& InPacketReader);

private:
	/** The network socket (incoming). */
	FSocket* Socket;

	/** The endpoint internet addr */
	TSharedPtr<FInternetAddr> EndpointInternetAddr;

	/** The thread object. */
	FRunnableThread* Thread;

	/** The receiver thread's name. */
	FString ThreadName;

	/** Used when any message is received (publish packet). */
	FOnMessage OnMessageDelegate;

	float SocketConnectionTimeout = 10.0f; // in seconds
	float MQTTConnectionTimeout = 10.0f; // in seconds
	double LastConnectRequestTime = 0.0f; // in seconds
	float KeepAliveInterval = 60.0f; // in seconds
	double LastKeepAliveTime = 0.0f;
};

template <class OperationType>
auto FMQTTConnection::QueueOperation(typename OperationType::FRequestType&& InRequestPacket)
	-> TFuture<typename OperationType::FResponseType>
{
	using RequestType = typename OperationType::FRequestType;
	using ResponseType = typename OperationType::FResponseType;

#if UE_BUILD_DEBUG
	if(GetState() == EMQTTState::Stopping)
	{
		UE_LOGF(LogMQTTCore, Warning, "Tried to queue operation while stopping: %ls", *OperationType::TypeName);
	}

	UE_LOGF(LogMQTTCore, Verbose, "Queued Operation: %ls", *OperationType::TypeName);
#endif

	TUniquePtr<OperationType> Operation = MakeUnique<OperationType>(MoveTemp(InRequestPacket), InRequestPacket.PacketId);
	auto Future = Operation->GetFuture();
	{
		FScopeLock Lock(&OutgoingQueueLock);
		OutgoingOperations.Enqueue(MoveTemp(Operation));
	}
	
	return Future;
}
