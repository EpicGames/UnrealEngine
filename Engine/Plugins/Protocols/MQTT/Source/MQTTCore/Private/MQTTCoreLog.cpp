// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTCoreLog.h"

DEFINE_LOG_CATEGORY(LogMQTTCore);
