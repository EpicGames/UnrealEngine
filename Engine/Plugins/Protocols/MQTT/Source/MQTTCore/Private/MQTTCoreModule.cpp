// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTCoreModule.h"

#include "GameDelegates.h"
#include "MQTTClient.h"
#include "MQTTClientSettings.h"
#include "MQTTCoreLog.h"
#include "Algo/Count.h"

#include "HAL/IConsoleManager.h"

void FMQTTCoreModule::StartupModule()
{
	RegisterConsoleCommands();
	FGameDelegates::Get().GetEndPlayMapDelegate().AddRaw(this, &FMQTTCoreModule::OnEndPlay);
}

void FMQTTCoreModule::ShutdownModule()
{
	FGameDelegates::Get().GetEndPlayMapDelegate().RemoveAll(this);
	UnregisterConsoleCommands();
	MQTTClients.Empty();
}

TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> FMQTTCoreModule::GetOrCreateClient(const bool bForceNew)
{
	const UMQTTClientSettings* Settings = GetDefault<UMQTTClientSettings>();
	return GetOrCreateClient(Settings->DefaultURL, bForceNew);
}

TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> FMQTTCoreModule::GetOrCreateClient(const FMQTTURL& InURL, const bool bForceNew)
{
	const FGuid ClientId = InURL.ToGuid();

	// Finds existing or creates new Client based on unique URL
	if(const TWeakPtr<IMQTTClient, ESPMode::ThreadSafe>* ExistingClient = MQTTClients.Find(ClientId))
	{
		if(!bForceNew)
		{
			return ExistingClient->Pin();
		}

		ExistingClient->Pin().Reset();
		MQTTClients.Remove(ClientId);
	}

	TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> CreatedClient = MQTTClients.Add(ClientId, MakeShared<FMQTTClient, ESPMode::ThreadSafe>(InURL)).Pin();
	ensure(MQTTClients.Num() > 0);
	UE_LOGF(LogMQTTCore, Display, "Created new Client, Num: %d", MQTTClients.Num());

	return CreatedClient;
}

void FMQTTCoreModule::RegisterConsoleCommands()
{ 
	ConsoleCommands.Add(MakeUnique<FAutoConsoleCommand>(
		TEXT("MQTT.GetOrCreateClient"),
		TEXT("Create the MQTT client"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FMQTTCoreModule::OnCreateClientCommand)
		));

	ConsoleCommands.Add(MakeUnique<FAutoConsoleCommand>(
		TEXT("MQTT.DestroyClient"),
		TEXT("Destroy the MQTT client"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FMQTTCoreModule::OnDestroyClientCommand)
		));

	ConsoleCommands.Add(MakeUnique<FAutoConsoleCommand>(
		TEXT("MQTT.ClientSubscribe"),
		TEXT("Subscribe on the topic"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FMQTTCoreModule::OnClientSubscribeCommand)
		));

	ConsoleCommands.Add(MakeUnique<FAutoConsoleCommand>(
		TEXT("MQTT.ClientUnsubscribe"),
		TEXT("Unsubscribe from the topic"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FMQTTCoreModule::OnClientUnsubscribeCommand)
		));

	ConsoleCommands.Add(MakeUnique<FAutoConsoleCommand>(
		TEXT("MQTT.ClientPublish"),
		TEXT("Publish to the topic"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FMQTTCoreModule::OnClientPublishCommand)
		));
}

void FMQTTCoreModule::UnregisterConsoleCommands()
{
	for (TUniquePtr<FAutoConsoleCommand>& Command : ConsoleCommands)
	{
		Command.Reset();
	}
}

void FMQTTCoreModule::OnCreateClientCommand(const TArray<FString>& InArgs)
{
	if (InArgs.Num() != 2)
	{
		return;
	}

	const FString& Host = InArgs[0];

	int32 Port = -1;
	LexFromString(Port, *InArgs[1]);

	if (Port == -1)
	{
		return;
	}

	GetOrCreateClient(FMQTTURL(Host, Port));
}

void FMQTTCoreModule::OnDestroyClientCommand(const TArray<FString>& InArgs)
{
	if (InArgs.Num() != 1)
	{
		return;
	}

	const FGuid ClientId = FGuid(InArgs[0]);

	UE_LOGF(LogMQTTCore, Display, "DestroyClient: %ls", *ClientId.ToString());

	// If the input is valid, this client *should* exist in MQTTClients
	ensureAlwaysMsgf(MQTTClients.Num() > 0, TEXT("Attempted to DestroyClient but MQTTClients was empty."));

	if(const TWeakPtr<IMQTTClient, ESPMode::ThreadSafe>* FoundClient = MQTTClients.Find(ClientId))
	{
		// if only 1 shared ref, it's the cached client unused by any active objects, so destroy
		if(FoundClient->Pin().GetSharedReferenceCount() <= 1)
		{
			MQTTClients.Remove(ClientId);	
		}
	}
}

void FMQTTCoreModule::OnClientSubscribeCommand(const TArray<FString>& InArgs)
{
	if (InArgs.Num() != 3)
	{
		return;
	}
	
	const FGuid ClientId = FGuid(InArgs[0]);
	if (!ClientId.IsValid())
	{
		return;
	}

	const FString SubscriptionPattern = InArgs[1];

	int32 QualityOfService = -1;
	LexFromString(QualityOfService, *InArgs[2]);
	if (QualityOfService == -1)
	{
		return;
	}
	
	const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = FindClient(ClientId);
	if (!MQTTClient.IsValid())
	{
		return;
	}

	MQTTClient->Subscribe({TPairInitializer<FString, EMQTTQualityOfService>{SubscriptionPattern, static_cast<EMQTTQualityOfService>(QualityOfService)}});
}

void FMQTTCoreModule::OnClientUnsubscribeCommand(const TArray<FString>& InArgs)
{
	if (InArgs.Num() != 2)
	{
		return;
	}
	
	const FGuid ClientId = FGuid(InArgs[0]);
	if (!ClientId.IsValid())
	{
		return;
	}

	const FString SubscriptionPattern = InArgs[1];

	const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = FindClient(ClientId);
	if (!MQTTClient.IsValid())
	{
		return;
	}

	MQTTClient->Unsubscribe({SubscriptionPattern});
}

void FMQTTCoreModule::OnClientPublishCommand(const TArray<FString>& InArgs)
{
	if (InArgs.Num() != 5)
	{
		return;
	}
	
	const FGuid ClientId = FGuid(InArgs[0]);
	if (!ClientId.IsValid())
	{
		return;
	}

	const FString Topic = InArgs[1];
	const FString Payload = InArgs[2];

	int32 QualityOfService = -1;
	LexFromString(QualityOfService, *InArgs[3]);
	if (QualityOfService == -1)
	{
		return;
	}

	bool bRetain = false;	
	LexFromString(bRetain, *InArgs[4]);

	const TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MQTTClient = FindClient(ClientId);
	if (!MQTTClient.IsValid())
	{
		return;
	}

	MQTTClient->Publish(Topic, Payload, static_cast<EMQTTQualityOfService>(QualityOfService), bRetain);
}

void FMQTTCoreModule::OnEndPlay()
{
	for(auto& MQTTClient : MQTTClients)
	{
		MQTTClient.Value.Reset();
	}
	MQTTClients.Empty();
}

int32 FMQTTCoreModule::GetClientNum() const
{
	// Only count valid clients (ignore stale pointers)
	return Algo::CountIf(MQTTClients,
	[](const TPair<FGuid, TWeakPtr<IMQTTClient, ESPMode::ThreadSafe>>& InIdClientPair)
	{
		return InIdClientPair.Value.IsValid();
	});
}

TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> FMQTTCoreModule::FindClient(const FGuid InId)
{
	check(InId.IsValid());

	if(TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> FoundClient = MQTTClients.Find(InId)->Pin())
	{
		return FoundClient;		
	}

	return nullptr; 
}

IMPLEMENT_MODULE(FMQTTCoreModule, MQTTCore);
