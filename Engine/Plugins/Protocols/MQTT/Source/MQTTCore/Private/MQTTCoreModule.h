// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMQTTCoreModule.h"
#include "Misc/Guid.h"

class FAutoConsoleCommand;

class FMQTTCoreModule : public IMQTTCoreModule
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin IMQTTCoreModule
	virtual TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> GetOrCreateClient(bool bForceNew = false) override;
	virtual TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> GetOrCreateClient(const FMQTTURL& InURL, bool bForceNew = false) override;
	//~ End IMQTTCoreModule

private:
	/** Register console commands. */
	void RegisterConsoleCommands();

	/** Unregister console commands. */
	void UnregisterConsoleCommands();

	void OnCreateClientCommand(const TArray<FString>& InArgs);	
	void OnDestroyClientCommand(const TArray<FString>& InArgs);

	void OnClientSubscribeCommand(const TArray<FString>& InArgs);
	void OnClientUnsubscribeCommand(const TArray<FString>& InArgs);
	void OnClientPublishCommand(const TArray<FString>& InArgs);

	void OnEndPlay();

protected:
	TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> FindClient(FGuid InId);
	virtual int32 GetClientNum() const override;

private:
	TMap<FGuid, TWeakPtr<IMQTTClient, ESPMode::ThreadSafe>> MQTTClients;

	/** Console commands handles. */
	TArray<TUniquePtr<FAutoConsoleCommand>> ConsoleCommands;
};
