// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTOperations.h"

#include "Algo/IndexOf.h"

REGISTER_MQTT_ARCHIVE(FMQTTFixedHeader)
REGISTER_MQTT_ARCHIVE(FMQTTConnectPacket)
REGISTER_MQTT_ARCHIVE(FMQTTConnectAckPacket)
REGISTER_MQTT_ARCHIVE(FMQTTPublishPacket)
REGISTER_MQTT_ARCHIVE(FMQTTPublishAckPacket)
REGISTER_MQTT_ARCHIVE(FMQTTPublishReceivedPacket)
REGISTER_MQTT_ARCHIVE(FMQTTPublishReleasePacket)
REGISTER_MQTT_ARCHIVE(FMQTTPublishCompletePacket)
REGISTER_MQTT_ARCHIVE(FMQTTSubscribePacket)
REGISTER_MQTT_ARCHIVE(FMQTTSubscribeAckPacket)
REGISTER_MQTT_ARCHIVE(FMQTTUnsubscribePacket)
REGISTER_MQTT_ARCHIVE(FMQTTUnsubscribeAckPacket)
REGISTER_MQTT_ARCHIVE(FMQTTPingRequestPacket)
REGISTER_MQTT_ARCHIVE(FMQTTPingResponsePacket)
REGISTER_MQTT_ARCHIVE(FMQTTDisconnectPacket)

namespace Internal
{
	static TMap<EMQTTPacketType, const TCHAR*> PacketTypeNames =
	{
		{ EMQTTPacketType::Connect, TEXT("Connect") },
		{ EMQTTPacketType::ConnectAcknowledge, TEXT("ConnectAcknowledge") },
			
		{ EMQTTPacketType::Publish, TEXT("Publish") },
		{ EMQTTPacketType::PublishAcknowledge, TEXT("PublishAcknowledge") },
		{ EMQTTPacketType::PublishReceived, TEXT("PublishReceive") },
		{ EMQTTPacketType::PublishRelease, TEXT("PublishRelease") },
		{ EMQTTPacketType::PublishComplete, TEXT("PublishComplete") },

		{ EMQTTPacketType::Subscribe, TEXT("Subscribe") },
		{ EMQTTPacketType::SubscribeAcknowledge, TEXT("SubscribeAcknowledge") },

		{ EMQTTPacketType::Unsubscribe, TEXT("Unsubscribe") },
		{ EMQTTPacketType::UnsubscribeAcknowledge, TEXT("UnsubscribeAcknowledge") },

		{ EMQTTPacketType::PingRequest, TEXT("PingRequest") },
		{ EMQTTPacketType::PingResponse, TEXT("PingResponse") },

		{ EMQTTPacketType::Disconnect, TEXT("Disconnect") },
		{ EMQTTPacketType::Authorize, TEXT("Authorize") },

		/** non-spec entries */
		{ EMQTTPacketType::None, TEXT("None") },
	};
}

const TCHAR* MQTT::GetMQTTPacketTypeName(EMQTTPacketType InMessageType)
{
	return Internal::PacketTypeNames[InMessageType];
}

template <typename SizeType = uint16>
void SerializeString(FArchive& Ar, const FString& InString)
{
	SizeType StringLength = InString.Len();
	const auto String = StringCast<ANSICHAR>(*InString);
	Ar << StringLength;
	Ar.Serialize((void*)String.Get(), StringLength);
}

template <typename SizeType = uint16>
void DeserializeString(FArchive& Ar, FString& OutString)
{
	SizeType StringLength;
	Ar << StringLength;
	TArray<char> Str;
	Str.Reserve(StringLength);
	Ar.Serialize(static_cast<void*>(Str.GetData()), StringLength);
	OutString = UTF8_TO_TCHAR(Str.GetData());
	OutString.LeftInline(StringLength); // @todo: fix, not sure why i get garbage end character	
}

/** Reads from Archive and writes to Header. */
uint8 FMQTTFixedHeader::ReadHeader(FArchive& InArchive, FMQTTFixedHeader& OutHeader)
{
	ensure(InArchive.IsLoading());
	
	InArchive.SetByteSwapping(true);

	InArchive << OutHeader.PacketType;

	uint32 MessageLength = 0;
	uint8 EncodedByte;
	unsigned int RemainingMultiplier = 1;
	for(int32 Idx = 0; Idx < 4; ++Idx)
	{
		if(!InArchive.AtEnd())
		{
			InArchive << EncodedByte;
			MessageLength += (EncodedByte & 127) * RemainingMultiplier;
			RemainingMultiplier *= 128;
			if((EncodedByte & 128) == 0)
			{
				OutHeader.RemainingLength = MessageLength;
				return OutHeader.LengthFieldSize = Idx + 1;
			}
		}
	}

	return false;
}

/** Reads from Header and writes to Archive. */
uint8 FMQTTFixedHeader::WriteHeader(FArchive& InArchive, const FMQTTFixedHeader& InHeader)
{
	ensure(InArchive.IsSaving());
	
	InArchive.SetByteSwapping(true);
	
	uint8 RemainingBytes[5];
	uint32 RemainingLength = InHeader.RemainingLength;
	uint32 RemainingCount = 0;
	do
	{
		uint8 EncodedByte = RemainingLength % 128;
		RemainingLength /= 128;
		if(RemainingLength > 0)
		{
			EncodedByte |= 128;
		}
		RemainingBytes[RemainingCount] = EncodedByte;
		RemainingCount++;
	}
	while (RemainingLength > 0 && RemainingCount < 5);
	
	ensure(RemainingCount != 5);

	uint8 PacketType = InHeader.PacketType;
	InArchive << PacketType;
	for(uint32 Idx = 0; Idx < RemainingCount; ++Idx)
	{
		InArchive << RemainingBytes[Idx];
	}

	return RemainingCount;
}

int64 FMQTTFixedHeader::HeaderSize() const
{
	return 1 + LengthFieldSize;
}

int64 FMQTTFixedHeader::TotalSize() const
{
	// PacketType + Length (varint) + Remaining
	return 1 + LengthFieldSize + RemainingLength;
}

void FMQTTFixedHeader::Serialize(FArchive& Ar)
{
	if(Ar.IsLoading())
	{
		ReadHeader(Ar, *this);
	}
	else if(Ar.IsSaving())
	{
		WriteHeader(Ar, *this);
	}
}

void FMQTTConnectPacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);

	if(Ar.IsLoading())
	{
		// ReadHeader(Ar, *this);
	}
	else if(Ar.IsSaving())
	{
		const bool bHasClientId = !ClientId.IsEmpty();
		const bool bHasUserName = !UserName.IsEmpty();
		const bool bHasPassword = !Password.IsEmpty();
		bool bHasWill = !WillTopic.IsEmpty();

		check(bHasPassword ? bHasUserName : true);

		constexpr uint32 HeaderLength = 10;
		uint32 PayloadLength = 0;

		PayloadLength += bHasClientId ? 2 + ClientId.Len() : 2;
		PayloadLength += bHasWill ? 2 + WillTopic.Len() + 2 + WillMessage.Len() : 0;
		PayloadLength += bHasUserName ? 2 + UserName.Len() : 0;
		PayloadLength += bHasPassword ? 2 + Password.Len() : 0;

		FixedHeader.SetPacketType(EMQTTPacketType::Connect);
		FixedHeader.RemainingLength = HeaderLength + PayloadLength;
		Ar << FixedHeader;

		SerializeString(Ar, "MQTT");

		uint8 ProtocolLevel = static_cast<uint8>(EMQTTProtocolLevel::MQTT311); // MQTT 3.11
		Ar << ProtocolLevel;

		uint8 Flags = (bCleanSession & 0x1) << 1;
		if(bHasWill)
		{
			Flags = Flags | ((static_cast<int32>(WillQoS) & 0x3) << 3 | (bHasWill & 0x1) << 2);
			if(bRetainWill)
			{
				Flags |= ((bRetainWill & 0x1) << 5);
			}
		}

		if(bHasUserName)
		{
			Flags |= 0x1 << 7;
		}

		if(bHasPassword)
		{
			Flags |= 0x1 << 6;
		}
	
		Ar << Flags;
		Ar << KeepAliveSeconds;

		// Payload
		if(bHasClientId)
		{
			SerializeString(Ar, ClientId);
		}
		else
		{
			static uint16 Zero = 0;
			Ar << Zero;
		}

		if(bHasWill)
		{
			SerializeString(Ar, WillTopic);
			SerializeString(Ar, WillMessage);
		}

		if(bHasUserName)
		{
			SerializeString(Ar, UserName);
			Flags |= 0x1 << 7;
		}

		if(bHasPassword)
		{
			SerializeString(Ar, Password);
			Flags |= 0x1 << 6;
		}
	}
}

void FMQTTConnectAckPacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);
	
	if(Ar.IsLoading())
	{
		Ar << FixedHeader;
		Ar << Flags;
		Ar << ReturnCode;
	}
	else if(Ar.IsSaving())
	{
	}
}

auto FMQTTPublishPacket::Serialize(FArchive& Ar) -> void
{
	Ar.SetByteSwapping(true);
	// @todo: disable bRetain if retain not available

	if(Ar.IsLoading())
	{
		Ar << FixedHeader;

		bIsDuplicate = (FixedHeader.PacketType & 0x08) >> 3;
		QoS = static_cast<EMQTTQualityOfService>((FixedHeader.PacketType & 0x06) >> 1);
		bRetain = (FixedHeader.PacketType & 0x01);

		const int64 HeaderEndPosition = Ar.Tell();

		DeserializeString(Ar, Topic);

		if(QoS > EMQTTQualityOfService::Once)
		{
			Ar << PacketId;
		}

		// Calculate variable header size (topic + optional packet id)
		const int64 VariableHeaderSize = Ar.Tell() - HeaderEndPosition;
		int32 PayloadLength = FixedHeader.RemainingLength - VariableHeaderSize;

		if(PayloadLength > 0)
		{
			Payload.Init(0, PayloadLength);
			for(int Idx = 0; Idx < PayloadLength; ++Idx)
			{
				Ar << Payload[Idx];
			}
		}
	}
	else if(Ar.IsSaving())
	{
		uint32 PacketLength = 2 + Payload.Num();
		if(!Topic.IsEmpty())
		{
			PacketLength += Topic.Len();	
		}

		FixedHeader.PacketType = static_cast<uint8>(static_cast<uint8>(EMQTTPacketType::Publish) | static_cast<uint8>(((bIsDuplicate ? 1 : 0) & 0x1) << 3) | (static_cast<uint8>(QoS) << 1) | static_cast<uint8>(bRetain ? 1 : 0));
		FixedHeader.RemainingLength = PacketLength + (QoS > EMQTTQualityOfService::Once ? 2 : 0);
		Ar << FixedHeader;

		if(!Topic.IsEmpty())
		{
			SerializeString(Ar, Topic);	
		}
		else
		{
			static uint16 Zero;
			Ar << Zero;	 
		}

		// [MQTT-2.3.1-5] of v3.1.1 spec
		if(QoS > EMQTTQualityOfService::Once)
		{
			Ar << PacketId;
		}

		if(Payload.Num() > 0)
		{
			for(int Idx = 0; Idx < Payload.Num(); ++Idx)
			{
				Ar << Payload[Idx];
			}
		}
	}
}

void FMQTTPublishAckPacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);

	if(Ar.IsLoading())
	{
		Ar << FixedHeader;
		Ar << PacketId;
	}
	else if(Ar.IsSaving())
	{
	}
}

void FMQTTPublishReceivedPacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);

	if(Ar.IsLoading())
	{
		Ar << FixedHeader;
		Ar << PacketId;
	}
	else if(Ar.IsSaving())
	{
		FixedHeader.PacketType = static_cast<uint8>(EMQTTPacketType::PublishReceived);
		FixedHeader.Reserved = 2;
		FixedHeader.RemainingLength = 2;
		Ar << FixedHeader;
		Ar << PacketId;
	}
}

void FMQTTPublishReleasePacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);
	
	if(Ar.IsLoading())
	{
		Ar << FixedHeader;
		Ar << PacketId;
	}
	else if(Ar.IsSaving())
	{
		FixedHeader.PacketType = static_cast<uint8>(EMQTTPacketType::PublishRelease) | 2;
		FixedHeader.Reserved = 2;
		FixedHeader.RemainingLength = 2;
		Ar << FixedHeader;
		Ar << PacketId;
	}
}

void FMQTTPublishCompletePacket::Serialize(FArchive& Ar)
{
	if(Ar.IsLoading())
	{
		Ar << FixedHeader;
		Ar << PacketId;
	}
	else if(Ar.IsSaving())
	{
		FixedHeader.PacketType = static_cast<uint8>(EMQTTPacketType::PublishComplete);
		FixedHeader.Reserved = 2;
		FixedHeader.RemainingLength = 2;
		Ar << FixedHeader;
		Ar << PacketId;
	}
}

void FMQTTSubscribePacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);

	if(Ar.IsLoading())
	{
	}
	else if(Ar.IsSaving())
	{
		// @todo: disable bRetain if retain not available	
		uint32 PacketLength = 2;
		for(FString& Topic : Topics)
		{
			ensure(Topic.Len() <= TNumericLimits<uint16>::Max());
			PacketLength += 2 + Topic.Len() + 1;
		}

		FixedHeader.PacketType = static_cast<uint8>(static_cast<uint8>(EMQTTPacketType::Subscribe) | 1 << 1);
		FixedHeader.RemainingLength = PacketLength;
		Ar << FixedHeader;
		Ar << PacketId;

		for(auto Idx = 0; Idx < Topics.Num(); ++Idx)
		{
			SerializeString(Ar, Topics[Idx]);
			Ar << RequestedQoS[Idx];
		}
	}
}

void FMQTTSubscribeAckPacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);

	if(Ar.IsLoading())
	{
		Ar << FixedHeader;
		Ar << PacketId;

		for(auto Idx = 0; Idx < FixedHeader.RemainingLength - 2; ++Idx)
		{
			Ar << ReturnCodes.Emplace_GetRef();
		}
	}
	else if(Ar.IsSaving())
	{

	}
}

void FMQTTUnsubscribePacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);

	if(Ar.IsLoading())
	{
	}
	else if(Ar.IsSaving())
	{
		// @todo: disable bRetain if retain not available	
		uint32 PacketLength = 2;
		for(FString& Topic : Topics)
		{
			ensure(Topic.Len() < TNumericLimits<uint16>::Max());
			PacketLength += 2 + Topic.Len();
		}

		FixedHeader.PacketType = static_cast<uint8>(static_cast<uint8>(EMQTTPacketType::Unsubscribe) | 1 << 1);
		FixedHeader.RemainingLength = PacketLength;
		Ar << FixedHeader;
		Ar << PacketId;

		for(FString& Topic : Topics)
		{
			SerializeString(Ar, Topic);
		}
	}
}

void FMQTTUnsubscribeAckPacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);

	if(Ar.IsLoading())
	{
		Ar << FixedHeader;		
		Ar << PacketId;
	}
	else if(Ar.IsSaving())
	{
		FixedHeader.PacketType = static_cast<uint8>(EMQTTPacketType::UnsubscribeAcknowledge);
		FixedHeader.Reserved = 2;
		FixedHeader.RemainingLength = 2;
		Ar << FixedHeader;
		Ar << PacketId;
	}
}

void FMQTTPingRequestPacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);

	if(Ar.IsLoading())
	{
		Ar << FixedHeader;
	}
	else if(Ar.IsSaving())
	{
		FixedHeader.PacketType = static_cast<uint8>(EMQTTPacketType::PingRequest);
		FixedHeader.Reserved = 2;
		FixedHeader.RemainingLength = 0;
		Ar << FixedHeader;
	}
}

void FMQTTPingResponsePacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);

	if(Ar.IsLoading())
	{
		Ar << FixedHeader;
	}
	else if(Ar.IsSaving())
	{
		FixedHeader.SetPacketType(EMQTTPacketType::PingResponse);
		FixedHeader.RemainingLength = 2;
		Ar << FixedHeader;
	}
}

void FMQTTDisconnectPacket::Serialize(FArchive& Ar)
{
	Ar.SetByteSwapping(true);

	if(Ar.IsLoading())
	{
		Ar << FixedHeader;
	}
	else if(Ar.IsSaving())
	{
		FixedHeader.PacketType = static_cast<uint8>(EMQTTPacketType::Disconnect);
		Ar << FixedHeader;
	}
}

FString FMQTTConnectOperation::TypeName = TEXT("FMQTTConnectOperation");
FString FMQTTDisconnectOperation::TypeName = TEXT("FMQTTDisconnectOperation");
FString FMQTTPublishOperationQoS0::TypeName = TEXT("FMQTTPublishOperationQoS0");
FString FMQTTPublishOperationQoS1::TypeName = TEXT("FMQTTPublishOperationQoS1");
FString FMQTTPublishOperationQoS2::TypeName = TEXT("FMQTTPublishOperationQoS2");
FString FMQTTPublishOperationQoS2_Step1::TypeName = TEXT("FMQTTPublishOperationQoS2_Step1");
FString FMQTTPublishOperationQoS2_Step2::TypeName = TEXT("FMQTTPublishOperationQoS2_Step2");
FString FMQTTPublishOperationQoS2_Step3::TypeName = TEXT("FMQTTPublishOperationQoS2_Step3");
FString FMQTTSubscribeOperation::TypeName = TEXT("FMQTTSubscribeOperation");
FString FMQTTUnsubscribeOperation::TypeName = TEXT("FMQTTUnsubscribeOperation");
FString FMQTTPingOperation::TypeName = TEXT("FMQTTPingOperation");

const FString& IMQTTOperation::GetTypeName() const
{
	static FString Empty;
	return Empty;
}

void FMQTTConnectOperation::Abandon()
{
	if(bCompleted.load() == false)
	{
		Promise.SetValue(FMQTTConnectAckPacket(EMQTTConnectReturnCode::SocketError));
	}
}

void FMQTTPublishOperationQoS2::Pack(FBufferArchive& InArchive)
{
	switch (CurrentPacketType)
	{
	case EMQTTPacketType::Publish:
		TMQTTOperation<FMQTTPublishPacket, FMQTTPublishCompletePacket>::Pack(InArchive);
		break;
		
	case EMQTTPacketType::PublishReceived:
		InArchive << PublishReceivedPacket;
		break;
		
	case EMQTTPacketType::PublishRelease:
		InArchive << PublishReleasePacket;
		break;

	default:
		checkNoEntry();
	}
}

uint16 FMQTTPublishOperationQoS2::CompleteStep(FMQTTPublishReceivedPacket&& InResponse)
{
	checkf(PendingPacketType == EMQTTPacketType::PublishReceived, TEXT("Unexpected packet type!"));
	PublishReceivedPacket = MoveTemp(InResponse);
	PublishReleasePacket.PacketId = PublishReceivedPacket.PacketId;
	NextStep();
	NextStep();
	return PacketId;
}

uint16 FMQTTPublishOperationQoS2::CompleteStep(FMQTTPublishReleasePacket&& InResponse)
{
	checkf(PendingPacketType == EMQTTPacketType::PublishRelease, TEXT("Unexpected packet type!"));
	PublishReleasePacket = MoveTemp(InResponse);
	NextStep();
	NextStep();
	return PacketId;
}

// get next request/response pair
void FMQTTPublishOperationQoS2::NextStep()
{
	const EMQTTPacketType NextPacketType = StepPacketTypes[Algo::IndexOf(StepPacketTypes, PendingPacketType) + 1];
	Swap(CurrentPacketType, PendingPacketType);
	PendingPacketType = NextPacketType;
}
