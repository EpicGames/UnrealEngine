// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MQTTProtocol.h"
#include "MQTTShared.h"
#include "MQTTSharedInternal.h"
#include "Serialization/BufferArchive.h" // IWYU pragma: keep

#pragma region Packets

struct FMQTTFixedHeader
{
	uint8 PacketType = static_cast<uint8>(EMQTTPacketType::Connect);
	uint8 Reserved = 0;
	uint8 LengthFieldSize = 1; // 1 - 4 depending on int precision
	uint16 RemainingLength = 0; // Length of variable header (10 bytes) + payload length
	uint16 TotalLength = 0;

	EMQTTPacketType GetPacketType() const
	{
		return static_cast<EMQTTPacketType>(PacketType);
	}

	void SetPacketType(EMQTTPacketType InPacketType)
	{
		PacketType = static_cast<uint8>(InPacketType);
	}

	int64 HeaderSize() const;
	int64 TotalSize() const;

	void Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FMQTTFixedHeader& InPacket);

protected:
	/** Reads from Archive and writes to Header. Returns size of Length field (1-4 bytes). */
	static uint8 ReadHeader(FArchive& InArchive, FMQTTFixedHeader& OutHeader);

	/** Reads from Header and writes to Archive. Returns size of Length field (1-4 bytes). */
	static uint8 WriteHeader(FArchive& InArchive, const FMQTTFixedHeader& InHeader);
};

struct FMQTTPacket
{
	// Not needed for certain packets, but practical for templating purposes
	uint16 PacketId = 0;
	
	FMQTTPacket() = default;	
	virtual ~FMQTTPacket() = default;

	/** Get's the "RemainingLength" field. Mainly used for testing/verification. */
	int32 GetLength() const { return FixedHeader.RemainingLength; }

protected:
	FMQTTFixedHeader FixedHeader;
};

struct FMQTTConnectPacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::Connect;

	explicit FMQTTConnectPacket(
		FString InClientId,
		const uint16 InKeepAliveSeconds = 60,
		FString InUserName = "",
		FString InPassword = "",
		const bool bInCleanSession = false,
		const bool bInRetainWill = false,
		FString InWillTopic = "",
		FString InWillMessage = "",
		const EMQTTQualityOfService InWillQoS = EMQTTQualityOfService::Once)
		: ClientId(MoveTemp(InClientId))
		, KeepAliveSeconds(InKeepAliveSeconds)
		, UserName(MoveTemp(InUserName))
		, Password(MoveTemp(InPassword))
		, bCleanSession(bInCleanSession)
		, bRetainWill(bInRetainWill)
		, WillTopic(MoveTemp(InWillTopic))
		, WillMessage(MoveTemp(InWillMessage))
		, WillQoS(InWillQoS)
	{
	}

	FString ClientId;
	uint16 KeepAliveSeconds = 60;
	FString UserName;
	FString Password;
	bool bCleanSession;
	bool bRetainWill;
	FString WillTopic;
	FString WillMessage;
	EMQTTQualityOfService WillQoS;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTConnectPacket& InPacket);
};

struct FMQTTConnectAckPacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::ConnectAcknowledge;

	FMQTTConnectAckPacket() = default;

	/** Short circuit when there's a connection error. */
	explicit FMQTTConnectAckPacket(const EMQTTConnectReturnCode& InReturnCode)
		: Flags(0)
		, ReturnCode(InReturnCode)
	{
	}

	uint8 Flags; // if clean session from client is 1, this should be 0, if from client is 0, this should be 1 if client session stored, or 0 if not
	EMQTTConnectReturnCode ReturnCode;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTConnectAckPacket& InPacket);
};

// @todo: for MQTT 5, properties, end of the message, pg25 of spec
struct FMQTTPublishPacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::Publish;

	FMQTTPublishPacket() = default;

	explicit FMQTTPublishPacket(
		const uint16 InPacketId,
		FString InTopic,
		const TArray<uint8> InPayload,
		const EMQTTQualityOfService InQoS,
		const bool bInRetain)
		: QoS(InQoS)
		, bRetain(bInRetain)
		, Topic(std::move(InTopic))
		, Payload(InPayload)
	{
		PacketId = InPacketId;
	}

	bool bIsDuplicate = false; // msg is a retry
	EMQTTQualityOfService QoS = EMQTTQualityOfService::Once;
	bool bRetain = false;
	FString Topic = ""; // utf8, no wildcards

	TArray<uint8> Payload; // RemainingLength - VariableLength, can be 0
	uint8 Response = 0; // depends on QoS, 0 = none, 1 = PUBACK, 2 = PUBREC

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTPublishPacket& InPacket);
};

struct FMQTTPublishAckPacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::PublishAcknowledge;
	uint8 Length = 2;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTPublishAckPacket& InPacket);
};

struct FMQTTPublishReceivedPacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::PublishReceived;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTPublishReceivedPacket& InPacket);
};

struct FMQTTPublishReleasePacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::PublishRelease;

	FMQTTPublishReleasePacket() = default;

	explicit FMQTTPublishReleasePacket(const uint16 InPacketId)
	{
		PacketId = InPacketId;
	}

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTPublishReleasePacket& InPacket);
};

struct FMQTTPublishCompletePacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::PublishComplete;
	FString Topic;
	TArray<uint8> Payload;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTPublishCompletePacket& InPacket);
};

struct FMQTTSubscribePacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::Subscribe;

	FMQTTSubscribePacket() = default;

	explicit FMQTTSubscribePacket(
		const uint16 InPacketId,
		const TArray<FString>& InTopics,
		const TArray<EMQTTQualityOfService>& InRequestedQoS)
		: Topics(InTopics)
		, RequestedQoS(InRequestedQoS)
	{
		PacketId = InPacketId;
	}

	TArray<FString> Topics; // utf8
	TArray<EMQTTQualityOfService> RequestedQoS;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTSubscribePacket& InPacket);
};

struct FMQTTSubscribeAckPacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::SubscribeAcknowledge;
	TArray<uint8> ReturnCodes;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTSubscribeAckPacket& InPacket);
};

struct FMQTTUnsubscribePacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::Unsubscribe;
	TArray<FString> Topics; // utf8

	FMQTTUnsubscribePacket() = default;

	explicit FMQTTUnsubscribePacket(
		const uint16 InPacketId,
		const TArray<FString>& InTopics)
		: Topics(InTopics)
	{
		PacketId = InPacketId;
	}

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTUnsubscribePacket& InPacket);
};

struct FMQTTUnsubscribeAckPacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::UnsubscribeAcknowledge;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTUnsubscribeAckPacket& InPacket);
};

struct FMQTTPingRequestPacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::PingRequest;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTPingRequestPacket& InPacket);
};

struct FMQTTPingResponsePacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::PingResponse;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTPingResponsePacket& InPacket);
};

struct FMQTTDisconnectPacket : FMQTTPacket
{
	constexpr static EMQTTPacketType PacketType = EMQTTPacketType::Disconnect;

	void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMQTTDisconnectPacket& InPacket);
};

#pragma endregion Packets

#pragma region Operations

/** An operation defines a message flow and encapsulates one or more packets. */
class IMQTTOperation
{
	constexpr static EMQTTPacketType NullPacketType = EMQTTPacketType::None;

public:
	explicit IMQTTOperation(
		const uint16 InPacketId = 0,
		const EMQTTState& InStateDependency = EMQTTState::Connected)
		: StateDependency(InStateDependency)
#if WITH_EDITORONLY_DATA
		, TimeQueued(FDateTime::UtcNow())
		, TimeExecuted(0)
#endif
		, PacketId(InPacketId)
		, bCompleted(false)
	{
	}

	virtual ~IMQTTOperation() = default;

	virtual EMQTTPacketType GetPacketType() const
	{
		return NullPacketType;
	}

	virtual EMQTTPacketType GetPendingPacketType() const
	{
		return NullPacketType;
	}

	const uint16& GetPacketId() const
	{
		return PacketId;
	}

	/** Serialize to provided archive. */
	virtual void Pack(FBufferArchive& InArchive)
	{
	}

	/** Flags task as complete and returns the PacketId, if any. */
	virtual uint16 Complete()
	{
		return PacketId;
	}

	/** Abandon task. */
	virtual void Abandon()
	{
	}

	/** Returns the operation type name for debug purposes. */
	virtual const FString& GetTypeName() const;

	EMQTTState StateDependency = EMQTTState::None;

#if WITH_EDITORONLY_DATA
	FDateTime TimeQueued;
	FDateTime TimeExecuted;

	uint32 GetTimeToExecution() const
	{
		return (TimeExecuted - TimeQueued).GetFractionMilli();
	}

	void MarkExecuted()
	{
		TimeExecuted = FDateTime::UtcNow();
	}
#endif

protected:
	uint16 PacketId = 0;
	std::atomic<bool> bCompleted;
};

/** Persisted as Session State. */
class IMQTTPersistentOperation
{
};

template <typename RequestType, typename ResponseType>
class TMQTTOperation : public IMQTTOperation
{
public:
	typedef RequestType FRequestType;
	typedef ResponseType FResponseType;

	explicit TMQTTOperation(
		RequestType&& InRequest,
		const uint16 InPacketId = 0,
		const EMQTTState& InStateDependency = EMQTTState::Connected)
		: IMQTTOperation(InPacketId, InStateDependency)
		, Packet(MoveTemp(InRequest))
	{
	}

	virtual ~TMQTTOperation() override;

	virtual EMQTTPacketType GetPacketType() const override
	{
		return RequestType::PacketType;
	}

	virtual EMQTTPacketType GetPendingPacketType() const override
	{
		return ResponseType::PacketType;
	}

	/** Serialize to provided archive. */
	virtual void Pack(FBufferArchive& InArchive) override
	{
		InArchive << Packet;
	}

	TFuture<ResponseType> GetFuture()
	{
		return Promise.GetFuture();
	}

	virtual uint16 Complete() override;
	virtual uint16 Complete(ResponseType&& InResponse);

	/** Abandon task. */
	virtual void Abandon() override;

protected:
	RequestType Packet;
	TPromise<ResponseType> Promise;
};

template <typename RequestType>
class TMQTTOperation<RequestType, void> : public IMQTTOperation
{
public:
	typedef RequestType FRequestType;
	typedef void FResponseType;
	explicit TMQTTOperation(
		RequestType&& InRequest,
		const uint16 InPacketId = 0,
		const EMQTTState& InStateDependency = EMQTTState::Connected)
		: IMQTTOperation(InPacketId, InStateDependency)
		, Packet(MoveTemp(InRequest))
	{
	}

	virtual EMQTTPacketType GetPacketType() const override
	{
		return RequestType::PacketType;
	}

	/** Serialize to provided archive. */
	virtual void Pack(FBufferArchive& InArchive) override
	{
		InArchive << Packet;
	}

	TFuture<void> GetFuture()
	{
		return Promise.GetFuture();
	}

	virtual uint16 Complete() override;

	/** Abandon task. */
	virtual void Abandon() override
	{
		if(bCompleted.load() == false)
		{
			Promise.SetValue();
		}
	}

protected:
	RequestType Packet;
	TPromise<void> Promise;
};

template <typename RequestType, typename ResponseType>
TMQTTOperation<RequestType, ResponseType>::~TMQTTOperation()
{
}

template <typename RequestType, typename ResponseType>
uint16 TMQTTOperation<RequestType, ResponseType>::Complete()
{
	bCompleted.store(true);
	Promise.SetValue({});
	return PacketId;
}

template <typename RequestType>
uint16 TMQTTOperation<RequestType, void>::Complete()
{
	bCompleted.store(true);
	Promise.SetValue();
	return PacketId;
}

template <typename RequestType, typename ResponseType>
uint16 TMQTTOperation<RequestType, ResponseType>::Complete(ResponseType&& InResponse)
{
	bCompleted.store(true);
	Promise.SetValue(MoveTemp(InResponse));
	return PacketId;
}

template <typename RequestType, typename ResponseType>
void TMQTTOperation<RequestType, ResponseType>::Abandon()
{
	if(bCompleted.load() == false)
	{
		Promise.SetValue({});
	}
}

class FMQTTConnectOperation : public TMQTTOperation<FMQTTConnectPacket, FMQTTConnectAckPacket>
{
public:
	explicit FMQTTConnectOperation(FMQTTConnectPacket&& InRequest, const uint16 InPacketId = 0)
		: TMQTTOperation<FMQTTConnectPacket, FMQTTConnectAckPacket>(MoveTemp(InRequest), InPacketId, EMQTTState::None)
	{
	}

	virtual void Abandon() override;

	static FString TypeName;
	virtual const FString& GetTypeName() const override { return TypeName; }
};

class FMQTTDisconnectOperation : public TMQTTOperation<FMQTTDisconnectPacket, void>
{
public:
	explicit FMQTTDisconnectOperation(FMQTTDisconnectPacket&& InRequest, const uint16 InPacketId = 0)
		: TMQTTOperation<FMQTTDisconnectPacket, void>(MoveTemp(InRequest), InPacketId, EMQTTState::None)
	{
	}

	static FString TypeName;
	virtual const FString& GetTypeName() const override { return TypeName; }
};

class FMQTTPublishOperationQoS0 : public TMQTTOperation<FMQTTPublishPacket, void>
{
public:
	explicit FMQTTPublishOperationQoS0(FMQTTPublishPacket&& InRequest, const uint16 InPacketId = 0)
		: TMQTTOperation<FMQTTPublishPacket, void>(MoveTemp(InRequest), InPacketId, EMQTTState::Connected)
	{
	}

	static FString TypeName;
	virtual const FString& GetTypeName() const override { return TypeName; }
};

class FMQTTPublishOperationQoS1 : public TMQTTOperation<FMQTTPublishPacket, FMQTTPublishAckPacket>
{
public:
	explicit FMQTTPublishOperationQoS1(FMQTTPublishPacket&& InRequest, const uint16 InPacketId = 0)
		: TMQTTOperation<FMQTTPublishPacket, FMQTTPublishAckPacket>(MoveTemp(InRequest), InPacketId, EMQTTState::Connected)
	{
	}

	static FString TypeName;
	virtual const FString& GetTypeName() const override { return TypeName; }
};

// Wraps all steps FMQTTPublishPacket -> FMQTTPublishCompletePacket
class FMQTTPublishOperationQoS2 final
	: public TMQTTOperation<FMQTTPublishPacket, FMQTTPublishCompletePacket>,
	public IMQTTPersistentOperation
{
public:
	explicit FMQTTPublishOperationQoS2(FMQTTPublishPacket&& InRequest, const uint16 InPacketId = 0)
		: TMQTTOperation<FMQTTPublishPacket, FMQTTPublishCompletePacket>(MoveTemp(InRequest), InPacketId, EMQTTState::Connected)
		, PublishReceivedPacket()
		, PublishReleasePacket()
	{
	}

	/** Serialize to provided archive. Packet serialized depends on current step. */
	virtual void Pack(FBufferArchive& InArchive) override;

	virtual EMQTTPacketType GetPacketType() const override
	{
		return CurrentPacketType;
	}

	virtual EMQTTPacketType GetPendingPacketType() const override
	{
		return PendingPacketType;
	}

	uint16 CompleteStep(FMQTTPublishReceivedPacket&& InResponse);
	uint16 CompleteStep(FMQTTPublishReleasePacket&& InResponse);

	static FString TypeName;
	virtual const FString& GetTypeName() const override { return TypeName; }

protected:
	static constexpr EMQTTPacketType StepPacketTypes[] = {EMQTTPacketType::PublishReceived, EMQTTPacketType::PublishRelease, EMQTTPacketType::PublishComplete};
	EMQTTPacketType CurrentPacketType = EMQTTPacketType::Publish;
	EMQTTPacketType PendingPacketType = EMQTTPacketType::PublishReceived;

	FMQTTPublishReceivedPacket PublishReceivedPacket;
	FMQTTPublishReleasePacket PublishReleasePacket;

	void NextStep();
};

// FMQTTPublishPacket -> FMQTTPublishReceivedPacket
class FMQTTPublishOperationQoS2_Step1
	: public TMQTTOperation<FMQTTPublishPacket, FMQTTPublishReceivedPacket>,
	public IMQTTPersistentOperation
{
public:
	explicit FMQTTPublishOperationQoS2_Step1(FMQTTPublishPacket&& InRequest, const uint16 InPacketId = 0)
		: TMQTTOperation<FMQTTPublishPacket, FMQTTPublishReceivedPacket>(MoveTemp(InRequest), InPacketId, EMQTTState::Connected)
	{
	}

	static FString TypeName;
	virtual const FString& GetTypeName() const override { return TypeName; }
};

// FMQTTPublishReceivedPacket -> FMQTTPublishReleasePacket
class FMQTTPublishOperationQoS2_Step2
	: public TMQTTOperation<FMQTTPublishReceivedPacket, FMQTTPublishReleasePacket>,
	public IMQTTPersistentOperation
{
public:
	explicit FMQTTPublishOperationQoS2_Step2(FMQTTPublishReceivedPacket&& InRequest, const uint16 InPacketId = 0)
		: TMQTTOperation<FMQTTPublishReceivedPacket, FMQTTPublishReleasePacket>(MoveTemp(InRequest), InPacketId, EMQTTState::Connected)
	{
	}

	static FString TypeName;
	virtual const FString& GetTypeName() const override { return TypeName; }
};

// FMQTTPublishReleasePacket -> FMQTTPublishCompletePacket
class FMQTTPublishOperationQoS2_Step3
	: public TMQTTOperation<FMQTTPublishReleasePacket, FMQTTPublishCompletePacket>,
	public IMQTTPersistentOperation
{
public:
	explicit FMQTTPublishOperationQoS2_Step3(FMQTTPublishReleasePacket&& InRequest, const uint16 InPacketId = 0)
		: TMQTTOperation<FMQTTPublishReleasePacket, FMQTTPublishCompletePacket>(MoveTemp(InRequest), InPacketId, EMQTTState::Connected)
	{
	}

	static FString TypeName;
	virtual const FString& GetTypeName() const override { return TypeName; }
};

class FMQTTSubscribeOperation : public TMQTTOperation<FMQTTSubscribePacket, FMQTTSubscribeAckPacket>
{
public:
	explicit FMQTTSubscribeOperation(FMQTTSubscribePacket&& InRequest, const uint16 InPacketId = 0)
		: TMQTTOperation<FMQTTSubscribePacket, FMQTTSubscribeAckPacket>(MoveTemp(InRequest), InPacketId, EMQTTState::Connected)
	{
	}
	
	static FString TypeName;
	virtual const FString& GetTypeName() const override { return TypeName; }
};

class FMQTTUnsubscribeOperation : public TMQTTOperation<FMQTTUnsubscribePacket, FMQTTUnsubscribeAckPacket>
{
public:
	explicit FMQTTUnsubscribeOperation(FMQTTUnsubscribePacket&& InRequest, const uint16 InPacketId = 0)
		: TMQTTOperation<FMQTTUnsubscribePacket, FMQTTUnsubscribeAckPacket>(MoveTemp(InRequest), InPacketId, EMQTTState::Connected)
	{
	}

	static FString TypeName;
	virtual const FString& GetTypeName() const override { return TypeName; }
};

class FMQTTPingOperation : public TMQTTOperation<FMQTTPingRequestPacket, FMQTTPingResponsePacket>
{
public:
	explicit FMQTTPingOperation(FMQTTPingRequestPacket&& InRequest, const uint16 InPacketId = 0)
		: TMQTTOperation<FMQTTPingRequestPacket, FMQTTPingResponsePacket>(MoveTemp(InRequest), InPacketId, EMQTTState::Connected)
	{
	}

	static FString TypeName;
	virtual const FString& GetTypeName() const override { return TypeName; }
};

#pragma endregion Operations
