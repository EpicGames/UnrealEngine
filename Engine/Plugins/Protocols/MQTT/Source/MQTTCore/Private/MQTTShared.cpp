// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTShared.h"

#include "SocketSubsystem.h"

#define LOCTEXT_NAMESPACE "MQTT"

bool FMQTTURL::GetAddress(ISocketSubsystem* InSocketSubsystem, TSharedPtr<FInternetAddr>& OutAddress) const
{
	OutAddress = InSocketSubsystem->CreateInternetAddr();

	// Probably Ip
	if(Host.Contains("."))
	{
		bool bIsValid = false;
		OutAddress->SetIp(*Host, bIsValid);
		if(!bIsValid)
		{
			return false;
		}
		OutAddress->SetPort(Port);
		return true;
	}
	// Probably hostname
	{
		FAddressInfoResult AddressInfo = InSocketSubsystem->GetAddressInfo(
			*Host,
			nullptr,
			EAddressInfoFlags::AllResultsWithMapping | EAddressInfoFlags::OnlyUsableAddresses,
			NAME_None);
		
		if (AddressInfo.ReturnCode == SE_NO_ERROR && AddressInfo.Results.Num() > 0)
		{
			// found, valid
			OutAddress = AddressInfo.Results[0].Address->Clone();
			OutAddress->SetPort(Port);
			return OutAddress->IsValid();
		}
		else
		{
			// todo: log error
		}
	}
	
	return false;
}

// @note: based on WaterUtils::StringToGuid()
FGuid FMQTTURL::ToGuid() const
{
	// Compute a 128-bit hash based on the string and use that as a GUID :
	const FTCHARToUTF8 Converted(*ToString());
	FMD5 MD5Gen;
	MD5Gen.Update((const uint8*)Converted.Get(), Converted.Length());
	uint32 Digest[4];
	MD5Gen.Final((uint8*)Digest);

	// FGuid::NewGuid() creates a version 4 UUID (at least on Windows), which will have the top 4 bits of the
	// second field set to 0100. We'll set the top bit to 1 in the GUID we create, to ensure that we can never
	// have a collision with other implicitly-generated GUIDs.
	Digest[1] |= 0x80000000;
	return FGuid(Digest[0], Digest[1], Digest[2], Digest[3]);
}

// Host is required, all others are optional
bool FMQTTURL::IsValid() const
{
	// User, Pass max length = 65535 bytes
	return !bFlaggedInvalid
		&& !Host.IsEmpty()	// Host must be set
		&& (!Username.IsEmpty() ? Username.Len() > TNumericLimits<uint16>::Max() - 2 : true)
		&& (!Password.IsEmpty() ? Password.Len() > TNumericLimits<uint16>::Max() - 2 : true)
		&& (!Password.IsEmpty() ? !Username.IsEmpty() : true); // Password needs Username too
}

bool FMQTTURL::IsValid(FText& OutMessage) const
{
	if(bFlaggedInvalid)
	{
		OutMessage = LOCTEXT("FlaggedInvalid", "URL has been flagged as invalid.");
		return false;
	}
		
	if(Host.IsEmpty())
	{
		OutMessage = LOCTEXT("InvalidHost", "Host (name or ip) is invalid.");
		return false;
	}

	if(!Password.IsEmpty() && Username.IsEmpty())
	{
		OutMessage = LOCTEXT("PasswordWithoutUsername", "Password set without a Username.");
		return false;
	}

	return true;
}

FMQTTURL FMQTTURL::FromString(const FString& InURL)
{
	FMQTTURL Result;
	FString ParsedStr = InURL;
	
	// Regex support isn't great, but here's the pattern: (?<Scheme>mqtt|mqtts|tcp|ssl)(?::\\/\\/)?(?<Username>\\w+)?(?<Password>(?::)\\w+)?@?(?<Host>\\w+)+:?(?<Port>\\d+)?

	// Scheme
	int32 DelimeterIdx = ParsedStr.Find(TEXT("://"));
	if(DelimeterIdx >= 0)
	{
		const FString SchemeStr = ParsedStr.Left(DelimeterIdx);

		static UEnum* SchemeEnum = FindObject<UEnum>(nullptr, TEXT("MQTTCore.EMQTTScheme"));
		int64 SchemeValueIdx = SchemeEnum->GetValueByNameString(SchemeStr);
		if(SchemeValueIdx != INDEX_NONE)
		{
			Result.Scheme = static_cast<EMQTTScheme>(SchemeValueIdx);	
		}

		ParsedStr.RightInline(ParsedStr.Len() - DelimeterIdx - 3);
	}

	// User+Pass
	DelimeterIdx = ParsedStr.Find(TEXT("@"));
	if(DelimeterIdx >= 0)
	{
		FString UserStr = ParsedStr.Left(DelimeterIdx);

		// Pass
		DelimeterIdx = UserStr.Find(TEXT(":"));
		if(DelimeterIdx >= 0)
		{
			const FString PassStr = UserStr.RightChop(DelimeterIdx + 1);
			Result.Password = PassStr;
			
			UserStr.LeftInline(DelimeterIdx);
		}

		Result.Username = UserStr;
		ParsedStr.RightInline(ParsedStr.Len() - DelimeterIdx - UserStr.Len() - 2);
	}

	// Host+Port
	DelimeterIdx = ParsedStr.Find(TEXT(":"));
	if(DelimeterIdx >= 0)
	{
		const FString PortStr = ParsedStr.Right(ParsedStr.Len() - DelimeterIdx - 1);
		Result.Port = FCString::Atoi64(*PortStr);

		ParsedStr.LeftInline(DelimeterIdx);
	}

	Result.Host = ParsedStr;

	return Result;
}

// mqtt[s]://[username][:password]@host.domain[:port]
FString FMQTTURL::ToString() const
{ 
	FString Result;

	// scheme ie. mqtt://
	Result.Append(UEnum::GetValueAsString(TEXT("MQTTCore.EMQTTScheme"), Scheme).ToLower() + "://");

	// user+pass
	if(!Username.IsEmpty())
	{
		Result.Append(Username);

		if(!Password.IsEmpty())
		{
			Result.Append(":" + Password);
		}

		Result.Append("@");
	}

	// hostname
	Result.Append(Host);

	// port
	if(Port > 0)
	{
		Result.Appendf(TEXT(":%i"), Port);
	}

	return Result;
}

void FMQTTTopic::MakeValid()
{
	Path.ReplaceInline(TEXT("\\"), TEXT("/")); // replace backslashes with forward
}

bool FMQTTTopic::IsValid(const FString& InTopic)
{
	if(InTopic.Len() == 0
		|| InTopic.Len() > 65535)
	{
		return false;
	}
	if(InTopic.Len() == 0
		|| InTopic.Contains(TEXT("\\")))
	{
		return false;
	}

	for(auto CharIdx = 0; CharIdx < InTopic.Len(); ++CharIdx)
	{
		if(InTopic[CharIdx] == FMQTTTopic::SingleLevelWildcardChar
			// if not first char last should be /
			&& ((CharIdx > 0 && CharIdx < InTopic.Len() - 1 && InTopic[CharIdx - 1] != '/')
			// or if not last char next should be /
			|| (CharIdx != InTopic.Len() - 1 && InTopic[CharIdx + 1] != '/')))
		{
			return false;
		}

		if(InTopic[CharIdx] == FMQTTTopic::MultiLevelWildcardChar
			// if not first char last should be /
			&& ((CharIdx > 0 && CharIdx < InTopic.Len() - 1 && InTopic[CharIdx + 1] != '/')
			// or if not last char
			|| (CharIdx != InTopic.Len() - 1)))
		{
			return false;
		}
	}

	return true;
}

bool FMQTTTopic::IsValid() const
{
	return FMQTTTopic::IsValid(Path);
}

FString FMQTTTopic::ToString() const
{
	return Path;
}

bool FMQTTTopicFilter::IsValid() const
{
	return FMQTTTopic::IsValid();
}

void FMQTTTopicFilter::MakeValid()
{
	FMQTTTopic::MakeValid();
}

FString FMQTTTopicFilter::ToString() const
{
	return FMQTTTopic::ToString();
}

bool FMQTTTopicTemplate::PopulateTemplate(const FStringFormatNamedArguments& InArgs, FMQTTTopic& OutTopic) const
{
	FString PopulatedPath = FString::Format(*Path, InArgs);
	OutTopic.Path = PopulatedPath;
	return true;
}

bool FMQTTSubscription::Matches(const FString& InTopic, const FString& InSubscription)
{
	size_t SubscriptionPos = 0;
	const size_t SubscriptionLength = InSubscription.Len();
	size_t TopicPos = 0;
	const size_t TopicLength = InTopic.Len();

	while (SubscriptionPos < SubscriptionLength && TopicPos < TopicLength)
	{
		// Char match
		if (InSubscription[SubscriptionPos] == InTopic[TopicPos])
		{
			// End of topic
			if (TopicPos == TopicLength - 1)
			{
				// Subscription leaf matches due to wildcard
				// Remaining chars are '/#'
				if ((SubscriptionPos == SubscriptionLength - 3
					&& InSubscription[SubscriptionPos + 1] == FMQTTTopic::TopicLevelSeparatorChar
					&& InSubscription[SubscriptionPos + 2] == FMQTTTopic::MultiLevelWildcardChar)

				// This char is '/', next is '#'
					|| (SubscriptionPos == SubscriptionLength - 2
					&& InSubscription[SubscriptionPos] == FMQTTTopic::TopicLevelSeparatorChar
					&& InSubscription[SubscriptionPos + 1] == FMQTTTopic::MultiLevelWildcardChar))
				{
					return true;
				}
			}

			SubscriptionPos++;
			TopicPos++;

			// At end of both, complete match
			if (SubscriptionPos == SubscriptionLength
				&& TopicPos == TopicLength)
			{
				return true;
			} 

			// End of topic, subscription ends with wildcard 
			if (TopicPos == TopicLength
				&& SubscriptionPos == SubscriptionLength - 1
				&& InSubscription[SubscriptionPos] == FMQTTTopic::SingleLevelWildcardChar)
			{
				// Subscription wildcard preceded by level separator
				return SubscriptionPos > 0 && InSubscription[SubscriptionPos - 1] == FMQTTTopic::TopicLevelSeparatorChar;
			}
		}
		// Char mismatch
		else
		{
			// SingleLevel Wildcard
			if (InSubscription[SubscriptionPos] == FMQTTTopic::SingleLevelWildcardChar)
			{
				// ...must be enclosed in level separator
				if ((SubscriptionPos > 0 && InSubscription[SubscriptionPos - 1] != FMQTTTopic::TopicLevelSeparatorChar)
					|| (SubscriptionPos < SubscriptionLength - 1 && InSubscription[SubscriptionPos + 1] != FMQTTTopic::TopicLevelSeparatorChar))
				{
					return false;
				}

				// Proceed until level separator found
				SubscriptionPos++;
				while (TopicPos < TopicLength && InTopic[TopicPos] != FMQTTTopic::TopicLevelSeparatorChar)
				{
					TopicPos++;
				}

				// Both topic and subscription end in level separator
				if(TopicPos == TopicLength && SubscriptionPos == SubscriptionLength)
				{
					return true;
				}
			}
			// MultiLevel Wildcard
			else if (InSubscription[SubscriptionPos] == FMQTTTopic::MultiLevelWildcardChar)
			{
				// ...must be preceded by level separator
				if (SubscriptionPos > 0
					&& InSubscription[SubscriptionPos - 1] != FMQTTTopic::TopicLevelSeparatorChar)
				{
					return false;
				}

				// ...and is the last character
				return SubscriptionPos + 1 == SubscriptionLength;
			}
			else
			{
				// Previous character was SingleLevel wildcard, remaining subscription is '/#'
				return SubscriptionPos > 0
					&& SubscriptionPos + 2 == SubscriptionLength
					&& TopicPos == TopicLength
					&& InSubscription[SubscriptionPos - 1] == FMQTTTopic::SingleLevelWildcardChar
					&& InSubscription[SubscriptionPos] == FMQTTTopic::TopicLevelSeparatorChar
					&& InSubscription[SubscriptionPos + 1] == FMQTTTopic::MultiLevelWildcardChar;
			}
		}
	}

	return false;
}

bool FMQTTSubscription::Matches(const FString& InTopic) const
{
	return FMQTTSubscription::Matches(InTopic, Topic.ToString());
}

bool FMQTTSubscription::IsValid() const
{
	return false;
}

#undef LOCTEXT_NAMESPACE // "MQTT"
