// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTSharedInternal.h"
#include "MQTTProtocol.h"

namespace Internal
{
	static TMap<ESocketConnectionState, const TCHAR*> SocketConnectionNames =
	{
		{ ESocketConnectionState::SCS_Connected, TEXT("Connected") },
		{ ESocketConnectionState::SCS_ConnectionError, TEXT("Connection Error") },
		{ ESocketConnectionState::SCS_NotConnected, TEXT("Not Connected") }
	};
			
	static TMap<EMQTTConnectReturnCode, const TCHAR*> ConnectReturnCodeDescriptions =
	{
		{ EMQTTConnectReturnCode::Accepted, TEXT("Connection accepted.") },
		{ EMQTTConnectReturnCode::RefusedProtocolVersion, TEXT("The Server does not the support the level of the MQTT protocol requested by the Client.") },
		{ EMQTTConnectReturnCode::RefusedIdentifierRejected, TEXT("The Client identifier is correct UTF-8 but not allowed by the Server.") },
		{ EMQTTConnectReturnCode::RefusedServerUnavailable, TEXT("The Network Connection has been made but the MQTT service is unavailable.") },
		{ EMQTTConnectReturnCode::RefusedBadUsernamePassword, TEXT("The data in the user name or password is malformed.") },
		{ EMQTTConnectReturnCode::RefusedNotAuthorized, TEXT("The Client is not authorized to connect.") },
		{ EMQTTConnectReturnCode::AlreadyConnected, TEXT("The Client is already connected to the Server.") },
		{ EMQTTConnectReturnCode::InvalidURL, TEXT("The provided URL is malformed.") },
		{ EMQTTConnectReturnCode::SocketError, TEXT("There was a socket error.") }
	};

	static TMap<EMQTTSocketState, const TCHAR*> SocketStateNames =
	{
		{ EMQTTSocketState::Connecting, TEXT("Connecting") },
		{ EMQTTSocketState::Connected, TEXT("Connected") },
		{ EMQTTSocketState::Error, TEXT("Error") },
		{ EMQTTSocketState::Stopping, TEXT("Stopping") },
		{ EMQTTSocketState::ConnectionRejected, TEXT("ConnectionRejected") },
	};
	
	static TMap<EMQTTState, const TCHAR*> StateNames =
	{
		{ EMQTTState::Unknown, TEXT("Unknown") },
		{ EMQTTState::Connected, TEXT("Connected") },
		{ EMQTTState::Connecting, TEXT("Connecting") },
		{ EMQTTState::Disconnected, TEXT("Disconnected") },
		{ EMQTTState::Disconnecting, TEXT("Disconnecting") },
		{ EMQTTState::Stopping, TEXT("Stopping") },
		{ EMQTTState::None, TEXT("None") }
	};

	static TMap<EMQTTSubscribeReturnCode, const TCHAR*> SubscribeReturnCodeNames =
	{
		{ EMQTTSubscribeReturnCode::Success_QoS0, TEXT("Quality of Service 0") },
		{ EMQTTSubscribeReturnCode::Success_QoS1, TEXT("Quality of Service 1") },
		{ EMQTTSubscribeReturnCode::Success_QoS2, TEXT("Quality of Service 2") },
		{ EMQTTSubscribeReturnCode::Failure, TEXT("Failure") },
	};
}

const TCHAR* MQTT::GetSocketConnectionStateName(ESocketConnectionState InConnectionState)
{
	return Internal::SocketConnectionNames[InConnectionState];
}

const TCHAR* MQTT::GetMQTTStateName(const EMQTTState InState)
{
	return Internal::StateNames[InState];
}

const TCHAR* MQTT::GetMQTTConnectReturnCodeDescription(const EMQTTConnectReturnCode InConnectReturnCode)
{
	return Internal::ConnectReturnCodeDescriptions[InConnectReturnCode];
}

const TCHAR* MQTT::GetMQTTSubscribeReturnCodeName(const EMQTTSubscribeReturnCode InSubscribeReturnCode)
{
	return Internal::SubscribeReturnCodeNames[InSubscribeReturnCode];
}
