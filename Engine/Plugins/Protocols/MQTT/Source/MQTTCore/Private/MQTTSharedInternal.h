// Copyright Epic Games, Inc. All Rights Reserved..

#pragma once

#include "SocketTypes.h"

enum class EMQTTConnectReturnCode : uint8;
enum class EMQTTPacketType : uint8;
enum class EMQTTSubscribeReturnCode : uint8;

#define REGISTER_MQTT_ARCHIVE(MQTTPacket)											\
FArchive& operator<<(FArchive & Ar, MQTTPacket &Packet)								\
{																					\
	Packet.Serialize(Ar);															\
	return Ar;																		\
}

#define REGISTER_MQTT_PACK(MQTTPacket)												\
bool MQTTPacket::Pack(FBufferArchive& InArchive)									\
{																					\
	InArchive << *this;																\
	return true;																	\
}

// #include "MQTTSharedInternal.generated.h"

/** Connection socket state */
enum class EMQTTSocketState : uint8
{
	Connecting = 0,
	Connected = 1,			// SCS_Connected
	Error = 2,				// Couldn't connect / SCS_ConnectionError
	ConnectionRejected = 3,	// Could connect, but was refused or rejected
	Stopping = 8,			// Force stop/shutdown
};

/** MQTT protocol state */
enum class EMQTTState : uint8
{
	Unknown = 0,
	Connected = 1,
	Connecting = 2,
	Disconnected = 3,
	Disconnecting = 4,
	Stopping = 8,			// Force stop/shutdown

	None = 127				// Used for state dependency to indicate there is none
};

namespace MQTT
{
	const TCHAR* GetSocketConnectionStateName(ESocketConnectionState InConnectionState);
	const TCHAR* GetMQTTPacketTypeName(EMQTTPacketType InMessageType);
	const TCHAR* GetMQTTStateName(EMQTTState InState);
	const TCHAR* GetMQTTConnectReturnCodeDescription(EMQTTConnectReturnCode InConnectReturnCode);
	const TCHAR* GetMQTTSubscribeReturnCodeName(EMQTTSubscribeReturnCode InSubscribeReturnCode);	
};

/** An arbitrary task allowing the storage/calls and ResultType to be separate (vs. a raw TPromise). */
class IMQTTTask
{
public:
	virtual ~IMQTTTask() = default;
	virtual void Execute() = 0;
	virtual void Abandon() = 0;

	EMQTTState StateDependency = EMQTTState::None;
};

template <typename ResultType>
class TMQTTTask final : public IMQTTTask
{
public:
	TMQTTTask(TUniqueFunction<ResultType()>&& InFunction, const EMQTTState& InStateDependency = EMQTTState::Connected)
		: Function(MoveTemp(InFunction))
	{
		StateDependency = InStateDependency;
	}

	virtual ~TMQTTTask() override
	{
	}

	virtual void Execute() override
	{
		Function();
	}

	virtual void Abandon() override
	{
		// Just complete the task
		Function();
	}

private:
	friend class FMQTTClientRunnable;

	/** The function to execute. */
	TUniqueFunction<ResultType()> Function; 
};

namespace MQTT
{
	template <typename CallableType>
	auto CreateTask(CallableType&& InCallable, const EMQTTState& InStateDependency) -> TSharedPtr<TMQTTTask<decltype(Forward<CallableType>(InCallable)())>>
	{
		using ResultType = decltype(Forward<CallableType>(InCallable)());
		TUniqueFunction<ResultType()> Function(Forward<CallableType>(InCallable));
		return MakeShared<TMQTTTask<ResultType>>(MoveTemp(Function), InStateDependency);
	}

	// template <typename RequestType, typename ResponseType>
	// auto CreateMessage(RequestType&& InRequest) -> TSharedPtr<TMQTTRequestResponseOperation<RequestType, ResponseType>>
	// {
	// 	return MakeShared<>()
	// 	using ResultType = decltype(Forward<CallableType>(InCallable)());
	// 	TUniqueFunction<ResultType()> Function(Forward<CallableType>(InCallable));
	// 	return MakeShared<TMQTTTask<ResultType>>(MoveTemp(Function), InStateDependency);
	// }
}
