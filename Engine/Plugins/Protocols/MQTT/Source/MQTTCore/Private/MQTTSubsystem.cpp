// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTSubsystem.h"

#include "JsonObjectWrapper.h"
#include "MQTTClientMessage.h"
#include "MQTTClientObject.h"
#include "MQTTClientSettings.h"
#include "Engine/Engine.h"

UMQTTClientObject* UMQTTSubsystem::GetOrCreateClient_WithProjectURL(UObject* InParent)
{
	const UMQTTClientSettings* Settings = GetDefault<UMQTTClientSettings>();
	return GetOrCreateClient(InParent, Settings->DefaultURL);
}

UMQTTClientObject* UMQTTSubsystem::GetOrCreateClient(UObject* InParent, const FMQTTURL& InURL)
{
	const FGuid ClientId = InURL.ToGuid();
	
	UMQTTSubsystem* Self = GEngine->GetEngineSubsystem<UMQTTSubsystem>();
	if(Self)
	{
		UMQTTClientObject* MQTTClientObject = Self->Clients.FindOrAdd(ClientId, NewObject<UMQTTClientObject>(InParent));

		// Might exist but be destroyed
		if(!MQTTClientObject || !IsValid(MQTTClientObject))
		{
			Self->Clients[ClientId] = MQTTClientObject = NewObject<UMQTTClientObject>(InParent);			
		}

		ensure(IsValid(MQTTClientObject));
		
		MQTTClientObject->Initialize(InURL);
		return MQTTClientObject;
	}

	return nullptr;
}

FString UMQTTSubsystem::GetPayloadString(const FMQTTClientMessage& InClientMessage)
{
	return InClientMessage.GetPayloadAsString();
}

bool UMQTTSubsystem::GetPayloadJson(UObject* InParent, const FMQTTClientMessage& InClientMessage, FJsonObjectWrapper& OutJson)
{
	TSharedPtr<FJsonObject> JsonObject;
	if(InClientMessage.GetPayloadAsJson(JsonObject))
	{
		OutJson.JsonObject = JsonObject;
	}

	return false;
}
