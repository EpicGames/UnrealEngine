// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MQTTProtocol.h"

#include "MQTTShared.h"

struct FMQTTURL;
struct FMQTTClientMessage;
struct FMQTTClientTask;
struct FMQTTSubscription;

class IMQTTClient
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnConnect, EMQTTConnectReturnCode)
	virtual FOnConnect& OnConnect() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnDisconnect)
	virtual FOnDisconnect& OnDisconnect() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnPublish)
	virtual FOnPublish& OnPublish() = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSubscribe, TArray<FMQTTSubscribeResult> /* SubscriptionHandles */)
	virtual FOnSubscribe& OnSubscribe() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnUnsubscribe)
	virtual FOnUnsubscribe& OnUnsubscribe() = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMessage, const FMQTTClientMessage& /* Packet */)
	virtual FOnMessage& OnMessage() = 0;	

public:
	virtual ~IMQTTClient() = default;

	// @todo: store session
	/** Connect using the clients URL. Set bCleanSession = false to resume previous session. */
	virtual TFuture<EMQTTConnectReturnCode> Connect(bool bCleanSession = true) = 0;

	virtual TFuture<void> Disconnect() = 0;

	/** Return true if the entire operation was successful (depends on QoS) */
	virtual TFuture<bool> Publish(const FString& InTopic, const TArray<uint8>& InPayload, EMQTTQualityOfService InQoS = EMQTTQualityOfService::Once, const bool bInRetain = false) = 0;

	/** Return true if the entire operation was successful (depends on QoS) */
	virtual TFuture<bool> Publish(const FString& InTopic, const FString& InPayload, EMQTTQualityOfService InQoS = EMQTTQualityOfService::Once, const bool bInRetain = false) = 0;

	virtual TFuture<TArray<FMQTTSubscribeResult>> Subscribe(const TArray<TPair<FString, EMQTTQualityOfService>>& InTopicFilterQoSPairs) = 0;

	// only for single sub
	template <typename CallableType>
	TFuture<FMQTTSubscribeResult> Subscribe(const FString& InTopicFilter, CallableType&& InOnMessage, EMQTTQualityOfService InQoS = EMQTTQualityOfService::Once);

	virtual TFuture<bool> Unsubscribe(const TSet<FString>& InTopicFilters) = 0;

	/** Returns true if response received before timeout. */
	virtual TFuture<bool> Ping(const float& InTimeout = 2.0f) = 0;

	/** Unique Id for this client. */
	virtual FGuid GetClientId() const = 0;

	/** URL for this client. */
	virtual const FMQTTURL& GetURL() const = 0;

	/** Is client currently connected? */
	virtual bool IsConnected() const = 0;
	
	/** Validity of this client. */
	virtual bool IsValid() const = 0;
};

template <typename CallableType>
TFuture<FMQTTSubscribeResult> IMQTTClient::Subscribe(const FString& InTopicFilter, CallableType&& InOnMessage, EMQTTQualityOfService InQoS)
{
	return Subscribe({MakeTuple(InTopicFilter, InQoS)})
		.Next([&](const TArray<FMQTTSubscribeResult>& InSubscribeResult)
		{
			if(InSubscribeResult.Num() == 0)
			{
				return FMQTTSubscribeResult(EMQTTSubscribeReturnCode::Failure, {});
			}

			if(InSubscribeResult[0].ReturnCode == EMQTTSubscribeReturnCode::Failure)
			{
				return InSubscribeResult[0];				
			}
			
			// Automatically subscribe to message using user-supplied function ...
			InSubscribeResult[0].Subscription->OnSubscriptionMessage().AddLambda(MoveTemp(InOnMessage));
			return InSubscribeResult[0];
		});
}

inline uint32 GetTypeHash(const IMQTTClient& InClient)
{
	return GetTypeHash(InClient.GetClientId());
}
