// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

struct FMQTTURL;
class IMQTTServer;
class IMQTTClient;

class IMQTTCoreModule : public IModuleInterface
{
public:
	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static IMQTTCoreModule& Get()
	{
		static const FName ModuleName = "MQTTCore";
		return FModuleManager::LoadModuleChecked<IMQTTCoreModule>(ModuleName);
	}

	/** Get or creates an MQTT client that uses the project default URL. An existing client is returned for a matching URL. */
	virtual TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> GetOrCreateClient(bool bForceNew = false) = 0;

	/** Creates an MQTT client from a URL. An existing client is returned for a matching URL. bForceNew = recreate if existing */
	virtual TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> GetOrCreateClient(const FMQTTURL& InURL, bool bForceNew = false) = 0;

protected:
#if WITH_EDITOR
#if WITH_DEV_AUTOMATION_TESTS
	friend class FMQTTClientSpec;
#endif
#endif

	virtual int32 GetClientNum() const = 0;
};
