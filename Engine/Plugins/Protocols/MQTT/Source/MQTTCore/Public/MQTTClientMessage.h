// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MQTTShared.h"
#include "Dom/JsonObject.h"

#include "MQTTClientMessage.generated.h"

USTRUCT(BlueprintType)
struct MQTTCORE_API FMQTTClientMessage
{
	GENERATED_BODY()

	/** TimeStamp as UTC. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MQTT")
	FDateTime TimeStamp = FDateTime::UtcNow();

	/** Packet topic. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MQTT")
	FString Topic;

	// @todo: func lib to get as string
	/** Packet content. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MQTT")
	TArray<uint8> Payload;

	/** Retain flag. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MQTT")
	bool bRetain = false;

	/** Quality of Service. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "MQTT", meta = (DisplayName = "Quality of Service"))
	EMQTTQualityOfService QoS = EMQTTQualityOfService::Once;

	/** Get's the payload as a string, ie. for Json. */
	const FString& GetPayloadAsString() const { return PayloadString; }

	/** Get's the payload as a Json object, first converting to string. Returns false if unsuccessful. */
	bool GetPayloadAsJson(TSharedPtr<FJsonObject>& OutJson) const;

	/** Set's the payload from the input string. */
	void SetPayloadFromString(const FString& InPayloadString);

	/** Gets the validity of the Packet. */
	bool IsValid() const { return true; }

protected:
	friend class FMQTTClient;
	
	/** Cached payload-as-string, ie. Json. */
	UPROPERTY()
	FString PayloadString;
};
