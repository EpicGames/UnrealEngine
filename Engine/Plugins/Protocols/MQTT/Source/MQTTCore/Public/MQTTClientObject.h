// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MQTTClientMessage.h"
#include "MQTTShared.h"

#include "MQTTClientObject.generated.h"

class IMQTTClient;

UCLASS(BlueprintType, meta = (DisplayName = "MQTT Subscription"))
class UMQTTSubscriptionObject final : public UObject
{
	GENERATED_BODY()
	
public:
	virtual ~UMQTTSubscriptionObject() override;
	
	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnMessageDelegate, const FMQTTClientMessage&, Message);

	/** Initialize from C++ subscription */
	void Initialize(const TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe>& InSubscription);

	UFUNCTION(BlueprintCallable, Category = "MQTT")
	void SetOnMessageHandler(const FOnMessageDelegate& OnMessageCallback);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MQTT")
	bool IsValid() const { return MqttSubscriptionPtr.IsValid(); }

	friend uint32 GetTypeHash(const UMQTTSubscriptionObject* InSubscription)
	{
		if(!::IsValid(InSubscription) || !InSubscription->IsValid())
		{
			return 0;
		}

		return GetTypeHash(*InSubscription->MqttSubscriptionPtr.Get());
	}

protected:
	UPROPERTY()
	FOnMessageDelegate OnMessageDelegate;

private:
	TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe> MqttSubscriptionPtr;

	void OnMessage(const FMQTTClientMessage& InMessage) const;
};

UCLASS(BlueprintType, meta = (DisplayName = "MQTT Client"))
class UMQTTClientObject final : public UObject
{	
	GENERATED_BODY()
	
public:
	virtual ~UMQTTClientObject() override;

	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnConnectDelegate, EMQTTConnectReturnCode, ReturnCode);
	DECLARE_DYNAMIC_DELEGATE(FOnDisconnectDelegate);
	DECLARE_DYNAMIC_DELEGATE(FOnPublishDelegate);	
	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnSubscribeDelegate, EMQTTSubscribeReturnCode, ReturnCode);
	DECLARE_DYNAMIC_DELEGATE(FOnUnsubscribeDelegate);
	DECLARE_DYNAMIC_DELEGATE_OneParam(FOnMessageDelegate, const FMQTTClientMessage&, Message);

	/** Initialize from URL */
	void Initialize(const FMQTTURL& InURL);
	
	void InitDelegates();
	void RemoveDelegates() const;

	virtual void BeginDestroy() override;

	TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> GetClientImpl() const;

	UFUNCTION(BlueprintCallable, Category = "MQTT")
	void Connect(UPARAM(DisplayName = "OnConnect") const FOnConnectDelegate& InOnConnect);

	UFUNCTION(BlueprintCallable, Category = "MQTT")
	void Disconnect(UPARAM(DisplayName = "OnDisconnect") const FOnDisconnectDelegate& InOnDisconnect);

	/** OutMessageId can be used to match this request with the callback response. */
	UFUNCTION(BlueprintCallable, Category = "MQTT")
	void Publish(
		UPARAM(DisplayName = "Topic") const FString& InTopic,
		UPARAM(DisplayName = "Payload") const TArray<uint8>& InPayload,
		UPARAM(DisplayName = "Quality of Service") EMQTTQualityOfService InQoS = EMQTTQualityOfService::Once,
		const bool bInRetain = false);

	/** OutMessageId can be used to match this request with the callback response. */
	UFUNCTION(BlueprintCallable, Category = "MQTT")
	UMQTTSubscriptionObject* Subscribe(
		UPARAM(DisplayName = "Topic") const FString& InTopic,
		UPARAM(DisplayName = "Quality of Service") EMQTTQualityOfService InQoS = EMQTTQualityOfService::Once);

	/** OutMessageId can be used to match this request with the callback response. */
	UFUNCTION(BlueprintCallable, Category = "MQTT", meta = (DisplayName = "Subscribe (Multiple Topics)"))
	TArray<UMQTTSubscriptionObject*> SubscribeMany(
		UPARAM(DisplayName = "Topics") const TArray<FString>& InTopics,
		UPARAM(DisplayName = "Quality of Services") const TArray<EMQTTQualityOfService> InQoS);

	/** OutMessageId can be used to match this request with the callback response. */
	UFUNCTION(BlueprintCallable, Category = "MQTT")
	void Unsubscribe(UPARAM(DisplayName = "Topic") const FString& InTopic);

	UFUNCTION(BlueprintPure, Category = "MQTT")
	FString GetClientId() const;

	/** URL for this client. */
	UFUNCTION(BlueprintPure, Category = "MQTT")
	FMQTTURL GetURL() const;

protected:
	UPROPERTY()
	FOnConnectDelegate OnConnectDelegate;
	
	UPROPERTY()
	FOnDisconnectDelegate OnDisconnectDelegate;

	UPROPERTY()
	FOnMessageDelegate OnMessageDelegate;

	UPROPERTY()
	TArray<TObjectPtr<UMQTTSubscriptionObject>> Subscriptions; 

private:
	TSharedPtr<IMQTTClient, ESPMode::ThreadSafe> MqttClientPtr;
};
