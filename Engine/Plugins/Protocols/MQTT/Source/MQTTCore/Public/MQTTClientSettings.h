// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MQTTShared.h"

#include "MQTTClientSettings.generated.h"

/**
* MQTT Client Settings
*/
UCLASS(Config = MQTT)
class MQTTCORE_API UMQTTClientSettings final : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Config, Category = "Connection")
	FMQTTURL DefaultURL;

	/** Maximum messages to publish per second. */
	UPROPERTY(EditAnywhere, Config, Category = "Bandwidth")
	uint32 PublishRate = 60;
};
