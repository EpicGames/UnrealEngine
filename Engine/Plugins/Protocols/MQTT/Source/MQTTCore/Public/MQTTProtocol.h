// Copyright Epic Games, Inc. All Rights Reserved.

// Types that correspond to the MQTT specification v3.1.1 and v5.0

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "MQTTProtocol.generated.h"

// @todo: max keepalive is 18h 12m 15s - MQTT spec v3.1.1 s 3.1.2.10

class FMQTTProtocol
{
};

#define PROTOCOL_NAME "MQTT"

#define PROTOCOL_VERSION_v311 4
#define PROTOCOL_VERSION_v5 5

enum class EMQTTProtocolLevel : uint8
{
	MQTT311 = 4,
	MQTT5 = 5
};

/* Control packet types */
enum class EMQTTPacketType : uint8
{
	Connect = 0x10U,
	ConnectAcknowledge = 0x20U,

	Publish = 0x30U,
	PublishAcknowledge = 0x40U,
	PublishReceived = 0x50U,
	PublishRelease = 0x60U,
	PublishComplete = 0x70U,

	Subscribe = 0x80U,
	SubscribeAcknowledge = 0x90U,

	Unsubscribe = 0xA0U,
	UnsubscribeAcknowledge = 0xB0U,

	PingRequest = 0xC0U,
	PingResponse = 0xD0U,

	Disconnect = 0xE0U,
	Authorize = 0xF0U,

	/** non-spec entries */
	None = 0,
};

/** MQTT v3.1.1 */
UENUM(BlueprintType)
enum class EMQTTConnectReturnCode : uint8
{
    /** Connection accepted. */
	Accepted = 0x00U,
    /** The Server does not the support the level of the MQTT protocol requested by the Client. */
	RefusedProtocolVersion = 0x01U,
    /** The Client identifier is correct UTF-8 but not allowed by the Server. */
	RefusedIdentifierRejected = 0x02U,
    /** The Network Connection has been made but the MQTT service is unavailable. */
	RefusedServerUnavailable = 0x03U,
    /** The data in the user name or password is malformed. */
	RefusedBadUsernamePassword = 0x04U,
    /** The Client is not authorized to connect. */
	RefusedNotAuthorized = 0x05U,

	/** Non-spec entries */
    /** The Client is already connected to the Server. */
	AlreadyConnected,
	/** The provided URL is malformed. */
	InvalidURL,
	/** Socket error. */
	SocketError
};

/** MQTT v3.1.1 */
UENUM(BlueprintType)
enum class EMQTTSubscribeReturnCode : uint8
{
    Success_QoS0 = 0x00U,           /* SubscribeAcknowledge */
    Success_QoS1 = 0x01U,           /* SubscribeAcknowledge */
    Success_QoS2 = 0x02U,           /* SubscribeAcknowledge */
    Failure = 0x80U                 /* SubscribeAcknowledge */
};

/** MQTT v5 (renamed from ReturnCode in v3.1.1) */
// @note: conflicting codes a differentiated by what they're returned from
enum class EMQTTReasonCode : uint8
{
	Success = 0x00U,						        /* ConnectAcknowledge, PublishAcknowledge, PublishReceive, PublishRelease, PublishComplete, UnsubscribeAcknowledge, Authorize */
	NormalDisconnection = 0x00U,			        /* Disconnect */
	Granted_QoS0 = 0x00U,					        /* SubscribeAcknowledge */
	Granted_QoS1 = 0x01U,					        /* SubscribeAcknowledge */
	Granted_QoS2 = 0x02U,					        /* SubscribeAcknowledge */
	DisconnectWithWillMsg = 0x04U,		            /* Disconnect */
	NoMatchingSubscribers = 0x10U,		            /* PublishAcknowledge, PublishReceive */
	NoSubscriptionExisted = 0x11U,		            /* UnsubscribeAcknowledge */
	ContinueAuthentication = 0x18U,					/* Authorize */
	ReAuthenticate = 0x19U,							/* Authorize */

	Unspecified = 0x80U,					        /* ConnectAcknowledge, PublishAcknowledge, PublishReceive, SubscribeAcknowledge, UnsubscribeAcknowledge, Disconnect */
	MalformedPacket = 0x19U,				        /* ConnectAcknowledge, Disconnect */
	ProtocolError = 0x82U,							/* Disconnect */
	ImplementationSpecific = 0x83U,					/* ConnectAcknowledge, PublishAcknowledge, PublishReceive, SubscribeAcknowledge, UnsubscribeAcknowledge, Disconnect */
	UnsupportedProtocolVersion = 0x84U,				/* ConnectAcknowledge */
	ClientIdNotValid = 0x85U,			            /* ConnectAcknowledge */
	BadUsernameOrPassword = 0x86U,					/* ConnectAcknowledge */
	NotAuthorized = 0x87U,							/* ConnectAcknowledge, PublishAcknowledge, PublishReceive, SubscribeAcknowledge, UnsubscribeAcknowledge, Disconnect */
	ServerUnavailable = 0x88U,						/* ConnectAcknowledge */
	ServerBusy = 0x89U,								/* ConnectAcknowledge, Disconnect */
	Banned = 0x8AU,									/* ConnectAcknowledge */
	ServerShuttingDown = 0x8BU,						/* Disconnect */
	BadAuthenticationMethod = 0x8CU,				/* ConnectAcknowledge */
	KeepAliveTimeout = 0x8DU,			            /* Disconnect */
	SessionTakenOver = 0x8EU,			            /* Disconnect */
	TopicFilterInvalid = 0x8FU,						/* SubscribeAcknowledge, UnsubscribeAcknowledge, Disconnect */
	TopicNameInvalid = 0x90U,			            /* ConnectAcknowledge, PublishAcknowledge, PublishReceive, Disconnect */
	PacketIdInUse = 0x91U,							/* PublishAcknowledge, SubscribeAcknowledge, UnsubscribeAcknowledge */
	PacketIdNotFound = 0x92U,			            /* PublishRelease, PublishComplete */
	ReceiveMaximumExceeded = 0x93U,					/* Disconnect */
	TopicAliasInvalid = 0x94U,						/* Disconnect */
	PacketTooLarge = 0x95U,							/* ConnectAcknowledge, PublishAcknowledge, PublishReceive, Disconnect */
	MessageRateTooHigh = 0x96U,						/* Disconnect */
	QuotaExceeded = 0x97U,							/* PublishAcknowledge, PublishReceive, SubscribeAcknowledge, Disconnect */
	AdministrativeAction = 0x98U,		            /* Disconnect */
	PayloadFormatInvalid = 0x99U,		            /* ConnectAcknowledge, Disconnect */
	RetainNotSupported = 0x9AU,						/* ConnectAcknowledge, Disconnect */
	QoSNotSupported = 0x9BU,			            /* ConnectAcknowledge, Disconnect */
	UseAnotherServer = 0x9CU,			            /* ConnectAcknowledge, Disconnect */
	ServerMoved = 0x9DU,					        /* ConnectAcknowledge, Disconnect */
	SharedSubscriptionNotSupported = 0x9EU,			/* SubscribeAcknowledge, Disconnect */
	ConnectionRateExceeded = 0x9FU,					/* ConnectAcknowledge, Disconnect */
	MaximumConnectTime = 0xA0U,						/* Disconnect */
	SubscriptionIdsNotSupported = 0xA1U,			/* SubscribeAcknowledge, Disconnect */
	WildcardSubscriptionNotSupported = 0xA2U,	    /* SubscribeAcknowledge, Disconnect */
};

enum class FMQTT5Property: uint8
{
	PayloadFormatIndicator = 1,		    /* Byte :				Publish, Will Properties */
	MessageExpiryInterval = 2,		    /* 4 Byte Int :			Publish, Will Properties */
	ContentType = 3,					/* Utf-8 String :		Publish, Will Properties */
	ResponseTopic = 8,				    /* Utf-8 String :		Publish, Will Properties */
	CorrelationData = 9,				/* Binary Data :		Publish, Will Properties */
	SubscriptionIdentifier = 11,		/* Variable Byte Int :	Publish, Subscribe */
	SessionExpiryInterval = 17,		    /* 4 Byte Int :			Connect, ConnectAcknowledge, Disconnect */
	AssignedClientIdentifier = 18,	    /* Utf-8 String :		ConnectAcknowledge */
	ServerKeepAlive = 19,			    /* 2 Byte Int :			ConnectAcknowledge */
	AuthenticationMethod = 21,		    /* Utf-8 String :		Connect, ConnectAcknowledge, Authorize */
	AuthenticationData = 22,			/* Binary Data :		Connect, ConnectAcknowledge, Authorize */
	RequestProblemInformation = 23,	    /* Byte :				Connect */
	WillDelayInterval = 24,			    /* 4 Byte Int :			Will Properties */
	RequestResponseInformation = 25,    /* Byte :				Connect */
	ResponseInformation = 26,		    /* Utf-8 String :		ConnectAcknowledge */
	ServerReference = 28,			    /* Utf-8 String :		ConnectAcknowledge, Disconnect */
	ReasonString = 31,				    /* Utf-8 String :		All Except Will Properties */
	ReceiveMaximum = 33,				/* 2 Byte Int :			Connect, ConnectAcknowledge */
	TopicAliasMaximum = 34,			    /* 2 Byte Int :			Connect, ConnectAcknowledge */
	TopicAlias = 35,					/* 2 Byte Int :			Publish */
	MaximumQoS = 36,					/* Byte :				ConnectAcknowledge */
	RetainAvailable = 37,			    /* Byte :				ConnectAcknowledge */
	UserProperty = 38,				    /* Utf-8 String Pair :	All */
	MaximumPacketSize = 39,			    /* 4 Byte Int :			Connect, ConnectAcknowledge */
	WildcardSubscriptionAvailable = 40, /* Byte :				ConnectAcknowledge */
	SubscriptionIdAvailable = 41,	    /* Byte :				ConnectAcknowledge */
	SharedSubscriptionAvailable = 42,	/* Byte :				ConnectAcknowledge */
};

enum class FMQTTPropertyType : uint8
{
	Byte = 1,
	Int16 = 2,
	Int32 = 3,
	VarInt = 4,
	Binary = 5,
	String = 6,
	StringPair = 7	
};

enum class FMQTTSubscriptionOption
{
	NoLocal = 0x04,				/* Won't receive own messages. */
	RetainAsPublished = 0x08,	/* */
	SendRetainAlways = 0x00,
	SendRetainNew = 0x10,
	SendRetainNever = 0x20	
};

/** 256MB MQTT v3.1.1: 2.2.3 */
constexpr static uint32 GMQTTMaxPayload = 268435455U;
