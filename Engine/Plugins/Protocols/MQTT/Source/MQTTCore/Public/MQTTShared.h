// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "CoreMinimal.h"
#include "IPAddress.h"
#include "MQTTProtocol.h"

#include "MQTTShared.generated.h"

struct FMQTTSubscription;
class IMQTTClient;
struct FMQTTClientMessage;
class ISocketSubsystem;

UENUM(BlueprintType)
enum class EMQTTScheme : uint8
{
	MQTT = 0,
	MQTTS = 1,		// Secure
	
	/* Possible but unsupported
	TCP = 2,
	SSL = 3
	*/
};

UENUM(BlueprintType)
enum class EMQTTQualityOfService : uint8
{
	Once = 0			UMETA(DisplayName = "QoS 0", Description = "Once (not guaranteed)"),
	AtLeastOnce = 1		UMETA(DisplayName = "QoS 1", Description = "At Least Once (guaranteed)"),
	ExactlyOnce = 2		UMETA(DisplayName = "QoS 2", Description = "Exactly Once (guaranteed)")
};

// mqtt[s]://[username][:password]@host.domain[:port]
USTRUCT(BlueprintType)
struct MQTTCORE_API FMQTTURL
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="URL")
	FString Host = TEXT("localhost");

	/** Default port is 1883 for MQTT scheme, 8883 for MQTTS. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category="URL")
	int32 Port = 1883;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="URL")
	FString Username;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="URL", meta = (EditCondition = "Username"))
	FString Password;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, AdvancedDisplay, Category="URL")
	EMQTTScheme Scheme = EMQTTScheme::MQTT;

	FMQTTURL() = default;

	/** Construct from components. */
	explicit FMQTTURL(
		const FString& InHost,
		const uint32 InPort = 1883,
		const FString& InUsername = TEXT(""),
		const FString& InPassword = TEXT(""),
		const EMQTTScheme InScheme = EMQTTScheme::MQTT)
		: Host(InHost),
		  Port(InPort),
		  Username(InUsername),
		  Password(InPassword),
		  Scheme(InScheme)
	{
	}

	/** */
	bool GetAddress(ISocketSubsystem* InSocketSubsystem, TSharedPtr<FInternetAddr>& OutAddress) const;

	/** Deterministic FGuid based on the URL. */
	FGuid ToGuid() const;

	/** Gets the validity of the URL. */
	bool IsValid() const;

	/** Gets the validity of the URL with message. */
	bool IsValid(FText& OutMessage) const;

	/** Flags this URL as invalid. */
	void SetInvalid() { bFlaggedInvalid = true; }

	/** Parses the input string. */
	static FMQTTURL FromString(const FString& InURL);

	/** Get the URL as a string. */
	FString ToString() const;

protected:
	bool bFlaggedInvalid = false;
};

inline uint32 GetTypeHash(const FMQTTURL& InURL)
{
	return GetTypeHash(InURL.ToString());
}

USTRUCT(BlueprintType)
struct MQTTCORE_API FMQTTTopic
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Topic")
	FString Path;

	FMQTTTopic() = default;

	/** Construct from path. */
	FMQTTTopic(const FString& InPath)
		: Path(InPath)
	{
	}

	virtual ~FMQTTTopic() = default;

	/** Gets the validity of the input Topic. */
	static bool IsValid(const FString& InTopic);
	
	/** Gets the validity of the Topic. */
	virtual bool IsValid() const;

	/** Attempts to make the topic valid. */
	virtual void MakeValid();

	/** Get the Topic as a string. */
	virtual FString ToString() const;

public:
	static constexpr TCHAR TopicLevelSeparatorChar = '/'; // U+002F

	/** '+' Must occupy entire level of the filter if it's used */
	static constexpr TCHAR SingleLevelWildcardChar = '+'; // U+002B
	
	/** '#' Must be specified either on its own or following a topic level separator, must be the last character in the Topic Filter */
	static constexpr TCHAR MultiLevelWildcardChar = '#'; // U+0023

	/** Not part of spec, but widely used by servers. */
	static constexpr TCHAR SystemChar = '$';
};

USTRUCT(BlueprintType)
struct MQTTCORE_API FMQTTTopicFilter : public FMQTTTopic
{
	GENERATED_BODY()
	
	FMQTTTopicFilter() {  }
	virtual ~FMQTTTopicFilter() override = default;

	/** Gets the validity of the TopicFilter. */
	virtual bool IsValid() const override;

	/** Attempts to make the topic TopicFilter. */
	virtual void MakeValid() override;

	/** Get the TopicFilter as a string. */
	virtual FString ToString() const override;
};

/** An MQTT Topic tokenized template */
USTRUCT(BlueprintType)
struct MQTTCORE_API FMQTTTopicTemplate : public FMQTTTopic
{
	GENERATED_BODY()

	bool PopulateTemplate(const FStringFormatNamedArguments& InArgs, FMQTTTopic& OutTopic) const;
	
};

struct MQTTCORE_API FMQTTSubscription
{
	/** Construct from path. */
	FMQTTSubscription(const FMQTTTopic& InTopic, const EMQTTQualityOfService InGrantedQoS)
		: Topic(InTopic)
		, GrantedQoS(InGrantedQoS)
	{
	}

	static bool Matches(const FString& InTopic, const FString& InSubscription);
	bool Matches(const FString& InTopic) const;

	const FMQTTTopic& GetTopic() const { return Topic; }

	/** Gets the validity of the Subscription. */
	bool IsValid() const;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSubscriptionMessage, const FMQTTClientMessage& /* InMessage */);
	FOnSubscriptionMessage& OnSubscriptionMessage() { return OnSubscriptionMessageDelegate; }

	EMQTTQualityOfService GetGrantedQoS() const { return GrantedQoS; }

	friend uint32 GetTypeHash(const FMQTTSubscription& InSubscription)
	{
		return GetTypeHash(InSubscription.Topic.ToString());
	}

private:
	friend uint32 GetTypeHash();
	
	FMQTTTopic Topic;
	EMQTTQualityOfService GrantedQoS;
	FOnSubscriptionMessage OnSubscriptionMessageDelegate;
};

struct MQTTCORE_API FMQTTSubscribeResult
{
	FMQTTSubscribeResult() = default;

	FMQTTSubscribeResult(
		const EMQTTSubscribeReturnCode InReturnCode,
		TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe> InSubscription)
		: ReturnCode(InReturnCode)
		, Subscription(MoveTemp(InSubscription))
	{
	}

	EMQTTSubscribeReturnCode ReturnCode;
	TSharedPtr<FMQTTSubscription, ESPMode::ThreadSafe> Subscription;
};
