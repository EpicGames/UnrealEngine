// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MQTTClientMessage.h"
#include "MQTTShared.h"
#include "Subsystems/EngineSubsystem.h"
#include "JsonObjectWrapper.h"

#include "MQTTSubsystem.generated.h"

class UMQTTServerObject;
class UMQTTClientObject;

/**
* Provides persistent access to clients
*/
UCLASS(BlueprintType)
class MQTTCORE_API UMQTTSubsystem final : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "MQTT", meta = (DefaultToSelf = "InParent", DisplayName = "Create Client (From Project URL)"))
	static UMQTTClientObject* GetOrCreateClient_WithProjectURL(UPARAM(DisplayName = "Parent") UObject* InParent);
	
	UFUNCTION(BlueprintCallable, Category = "MQTT", meta = (DefaultToSelf = "InParent", DisplayName = "Create Client (From URL)"))
	static UMQTTClientObject* GetOrCreateClient(UPARAM(DisplayName = "Parent") UObject* InParent, UPARAM(DisplayName = "URL") const FMQTTURL& InURL);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MQTT")
	static FString GetPayloadString(UPARAM(DisplayName = "ClientMessage") const FMQTTClientMessage& InClientMessage);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "MQTT", meta = (DefaultToSelf = "InParent"))
	static bool GetPayloadJson(UPARAM(DisplayName = "Parent") UObject* InParent, UPARAM(DisplayName = "ClientMessage") const FMQTTClientMessage& InClientMessage, FJsonObjectWrapper& OutJson);

private:
	UPROPERTY(Transient)
	TMap<FGuid, TObjectPtr<UMQTTClientObject>> Clients;
};
