// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MQTTCoreEditor : ModuleRules
{
	public MQTTCoreEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new[] {
				"Core",
                "CoreUObject"
            });

		PrivateDependencyModuleNames.AddRange(
			new[] {
				"Engine",
				"MQTTCore",
				"Settings",
				"Slate",
				"SlateCore",
				"UnrealEd"
			});
    }
}
