// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTCoreEditorLog.h"

DEFINE_LOG_CATEGORY(LogMQTTCoreEditor);
