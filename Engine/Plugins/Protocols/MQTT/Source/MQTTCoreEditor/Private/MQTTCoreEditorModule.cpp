// Copyright Epic Games, Inc. All Rights Reserved.

#include "MQTTCoreEditorModule.h"

#include "ISettingsModule.h"
#include "MQTTClientSettings.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "MQTTCoreEditor"

void FMQTTCoreEditorModule::StartupModule()
{
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	// Register global settings
	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "MQTT",
										LOCTEXT("ProjectSettings_Label", "MQTT"),
										LOCTEXT("ProjectSettings_Description", "Configure MQTT settings"),
										GetMutableDefault<UMQTTClientSettings>()
		);
	}
}

void FMQTTCoreEditorModule::ShutdownModule()
{
	// Unregister global settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule)
	{
		SettingsModule->UnregisterSettings("Project", "Plugins", "MQTT");
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FMQTTCoreEditorModule, MQTTCoreEditor)
