// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMQTTCoreEditorModule.h"


class FMQTTCoreEditorModule final
    : public IMQTTCoreEditorModuleInterface
{
public:
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
};
